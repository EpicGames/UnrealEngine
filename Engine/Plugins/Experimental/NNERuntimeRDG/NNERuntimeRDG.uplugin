{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "NNERuntimeRDG",
	"Description": "A runtime implementing the Neural Network Engine (NNE) API, using the Render Dependency Graph (RDG).",
	"Category": "ML",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NNEHlslShaders",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "NNERuntimeRDG",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNERuntimeRDGData",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNERuntimeRDGUtils",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		}
	]
}