// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define WORK_TYPE float

Buffer<WORK_TYPE> X;
Buffer<WORK_TYPE> Scales;
Buffer<WORK_TYPE> Bias;
Buffer<WORK_TYPE> Mean;
Buffer<WORK_TYPE> Var;
RWBuffer<WORK_TYPE> Output;
float Epsilon;
uint SpatialVolume;
uint DimC;
uint ThreadCountX;
uint Num;

WORK_TYPE ApplyBatchNormalization(WORK_TYPE x, WORK_TYPE input_mean, WORK_TYPE input_var, WORK_TYPE scale, WORK_TYPE bias)
{
	//https://github.com/onnx/onnx/blob/main/docs/Operators.md#BatchNormalization
	return (x - input_mean) / sqrt(input_var + Epsilon) * scale + bias;

}


[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void BatchNormalization(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint IndexX = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	const uint IndexOther = (IndexX / SpatialVolume) % DimC;
	if (IndexX < Num)
	{
		float x = X[IndexX];
		float mean = Mean[IndexOther];
		float var = Var[IndexOther];
		float scale = Scales[IndexOther];
		float bias = Bias[IndexOther];
		Output[IndexX] = ApplyBatchNormalization(x, mean, var, scale, bias);
	}
}
