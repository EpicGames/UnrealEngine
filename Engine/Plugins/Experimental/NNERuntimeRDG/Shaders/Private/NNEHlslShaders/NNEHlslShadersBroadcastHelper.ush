// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

void DivMod(uint op1, uint op2, out uint q, out uint r)
{
	q = op1 / op2;
	r = op1 - (q * op2);
}