// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

Buffer<INPUT_TYPE> Input;
RWBuffer<OUTPUT_TYPE> Output;
uint Num;
uint ThreadCountX;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Cast(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	if (Index < Num)
	{
		Output[Index] = (OUTPUT_TYPE) Input[Index];
	}
}
