// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define BUFFER_TO_WORK_TYPE(x) x
#define WORK_TO_BUFFER_TYPE(x) x

#define LDS_IC 16 //Input channel cached in LDS
#define TG_OC 32 //Number of output channel covered by a thread group
#define TG_OP 32 //Number of output pixel covered by a thread group

//To avoid bank conflict when writing out to DDR
#define LDS_PADX 1
#define LDS_PADY 2
#define LDS_X_PADDEDSIZE ((4 + LDS_PADX) * 8 + LDS_PADY) //42

groupshared WORK_TYPE LDSBuffer[32 * LDS_X_PADDEDSIZE]; //max (16*32+32,32*42) = 1344 WORK_TYPE 

Buffer<BUFFER_TYPE> Input; // Ni x Ci x Hi x Wi
Buffer<BUFFER_TYPE> Weight; // Cw x Ci x Hw x Ww OR Hw x Ww x Ci x Cw if WEIGHTS_TRANSPOSED
Buffer<BUFFER_TYPE> Bias; // Cw
RWBuffer<BUFFER_TYPE> Output; // Ni x Cw x Ho x Wo

int Ci;
int Hi;
int Wi;
int Ho;
int Wo;
int Cw;
int Hw;
int Ww;
int PadLeft;
int PadTop;
int StrideH;
int StrideW;

WORK_TYPE ReadLDSWeights(int inputChannel, int outputChannel)
{
	return LDSBuffer[LDS_IC * TG_OP + inputChannel * TG_OC + outputChannel];
}

void WriteLDSWeights(int inputChannel, int outputChannel, WORK_TYPE value)
{
	LDSBuffer[LDS_IC * TG_OP + inputChannel * TG_OC + outputChannel] = value;
}

WORK_TYPE ReadLDSInputs(int inputChannel, int outputPixel)
{
	return LDSBuffer[inputChannel * TG_OP + outputPixel];
}

void WriteLDSInputs(int inputChannel, int outputPixel, WORK_TYPE value)
{
	LDSBuffer[inputChannel * TG_OP + outputPixel] = value;
}

[numthreads(8, 8, 1)]
void ConvMatmul(
	in const uint3 DispatchThreadID : SV_DispatchThreadID,
	in const uint3 GroupID : SV_GroupID,
	in const uint3 GroupThreadID : SV_GroupThreadID,
	in const uint GroupIndex : SV_GroupIndex)
{
	const int DispatchThreadOutputPixelOffset = 4 * DispatchThreadID.x;
	const int DispatchThreadOutputChannelOffset = 4 * DispatchThreadID.y;
	const int GroupThreadOutputPixelOffset = 4 * GroupThreadID.x;
	const int GroupThreadOutputChannelOffset = 4 * GroupThreadID.y;
	const int GroupThreadIdx = GroupIndex;
	const int Scalar_GroupOutputPixelOffset = TG_OP * GroupID.x;
	const int Scalar_GroupOutputChannelOffset = TG_OC * GroupID.y;
	const int Scalar_BatchOutputOffset = Ho * Wo * Cw * GroupID.z;
	const int Scalar_BatchInputOffset = Hi * Wi * Ci * GroupID.z;
	const int Scalar_GroupOutputPixelH = (Scalar_GroupOutputPixelOffset) / Wo * StrideH;
	const int Scalar_GroupOutputPixelWBase = Scalar_GroupOutputPixelOffset % Wo * StrideW;
	const int Scalar_GroupInputPixelKernelTopLeftH = Scalar_GroupOutputPixelH - PadTop;
	const int Scalar_GroupInputPixelKernelTopLeftW = Scalar_GroupOutputPixelWBase - PadLeft;

	const int cw = DispatchThreadOutputChannelOffset;
	const int pi = DispatchThreadOutputPixelOffset;

#if HAS_BIAS
	WORK_TYPE BiasC0 = BUFFER_TO_WORK_TYPE(Bias[cw+0]);
	WORK_TYPE BiasC1 = BUFFER_TO_WORK_TYPE(Bias[cw+1]);
	WORK_TYPE BiasC2 = BUFFER_TO_WORK_TYPE(Bias[cw+2]);
	WORK_TYPE BiasC3 = BUFFER_TO_WORK_TYPE(Bias[cw+3]);
#else
	WORK_TYPE BiasC0 = 0.0f;
	WORK_TYPE BiasC1 = 0.0f;
	WORK_TYPE BiasC2 = 0.0f;
	WORK_TYPE BiasC3 = 0.0f;
#endif
	
	int i;
	// First index is thread output channel offset (on cw)
	// 2nd index is thread pixels offset (on pi)
	WORK_TYPE Values[4][4];
	
	UNROLL
	for (i = 0; i < 4; ++i)
	{
		Values[0][i] = BiasC0;
		Values[1][i] = BiasC1;
		Values[2][i] = BiasC2;
		Values[3][i] = BiasC3;
	}
	
	int Scalar_KernelIdx = 0;
	const int InputChannelOffset = GroupThreadIdx >= 32 ? LDS_IC/2 : 0;
	const int OutputPixelOffset = GroupThreadIdx % 32;
	const int OutputChannelOffset = GroupThreadIdx % 32;
	int OutputChannel = Scalar_GroupOutputChannelOffset + OutputChannelOffset;
	const bool IsValidOutputChannel = OutputChannel < Cw;
	OutputChannel = min(OutputChannel, Cw - 1);
	
	for (int hw = 0; hw < Hw; ++hw)
	{
		const int Scalar_InputPixelH = Scalar_GroupInputPixelKernelTopLeftH + hw;
		const bool Scalar_IsValidInputH = (Scalar_InputPixelH >= 0) && (Scalar_InputPixelH < Hi);
		
		for (int ww = 0; ww < Ww; ++ww)
		{
			const int InputPixelW = Scalar_GroupInputPixelKernelTopLeftW + ww + OutputPixelOffset * StrideW;
			const bool IsValidInputPixel = Scalar_IsValidInputH && (InputPixelW >= 0) && (InputPixelW < Wi);
			int InputPixelOffset = Scalar_InputPixelH * Wi + InputPixelW;
			InputPixelOffset = clamp(InputPixelOffset, 0, Hi * Wi - 1);
			
			const int ReadOffsetI = Scalar_BatchInputOffset + InputPixelOffset;

			#if WEIGHTS_TRANSPOSED
			const int Scalar_InputChannelStride = Cw;
			const int ReadOffsetW = Scalar_KernelIdx + OutputChannel;
			#else
			const int Scalar_InputChannelStride = Hw * Ww;
			const int ReadOffsetW = OutputChannel * Ci * Scalar_InputChannelStride + Scalar_KernelIdx;
			#endif
			
			for (int ci = 0; ci < Ci; ci += LDS_IC)
			{
				///— DDR → LDS
				// We need to load 1024 floats
				// 512 weights (16 inputs channel, 32 output channels) and
				// 512 inputs (16 inputs channel, 32 inputs pixels)
				// We have 64 threads thus each will read 8 inputs and 8 weights.

				
				UNROLL
				for (i = 0; i < (LDS_IC / 2); ++i)
				{
					int InputChannel = ci + InputChannelOffset + i;
					const bool IsValidInputChannel = InputChannel < Ci;
					InputChannel = min(InputChannel, Ci - 1);

					WORK_TYPE ValueI = BUFFER_TO_WORK_TYPE(Input[Hi * Wi * InputChannel + ReadOffsetI]);
					ValueI = IsValidInputChannel && IsValidInputPixel ? ValueI : 0.0f;
					WriteLDSInputs(InputChannelOffset + i, OutputPixelOffset, ValueI);

					WORK_TYPE ValueW = BUFFER_TO_WORK_TYPE(Weight[Scalar_InputChannelStride * InputChannel + ReadOffsetW]);
					ValueW = IsValidInputChannel && IsValidOutputChannel ? ValueW : 0.0f;
					WriteLDSWeights(InputChannelOffset + i, OutputChannelOffset, ValueW);
				}
				
				GroupMemoryBarrierWithGroupSync();
				
				/// LDS to register + inner loop
				// Loop on cached input channels
				#define INNERLOOPUNROLLCOUNT 4
				for (int cachedCiBase = 0; cachedCiBase < LDS_IC; cachedCiBase += INNERLOOPUNROLLCOUNT)
				{
					WORK_TYPE RegWeights[INNERLOOPUNROLLCOUNT][4];
					WORK_TYPE RegInputs[INNERLOOPUNROLLCOUNT][4];
					
					UNROLL
					for (int unrolledIdx = 0; unrolledIdx < INNERLOOPUNROLLCOUNT; ++unrolledIdx)
					{
						// 8 load from LDS
						UNROLL
						for (i = 0; i < 4; ++i)
						{
							RegWeights[unrolledIdx][i] = ReadLDSWeights(cachedCiBase + unrolledIdx, GroupThreadOutputChannelOffset + i);
							RegInputs[unrolledIdx][i] = ReadLDSInputs(cachedCiBase + unrolledIdx, GroupThreadOutputPixelOffset + i);
						}

						// Inner loop (16 mads)
						UNROLL
						for (i = 0; i < 4; ++i)
						{
							Values[i][0] += RegWeights[unrolledIdx][i] * RegInputs[unrolledIdx][0];
							Values[i][1] += RegWeights[unrolledIdx][i] * RegInputs[unrolledIdx][1];
							Values[i][2] += RegWeights[unrolledIdx][i] * RegInputs[unrolledIdx][2];
							Values[i][3] += RegWeights[unrolledIdx][i] * RegInputs[unrolledIdx][3];
						}
					}
				}
				
				GroupMemoryBarrierWithGroupSync();
			}
			#if WEIGHTS_TRANSPOSED
				Scalar_KernelIdx += Ci * Cw;
			#else
			++Scalar_KernelIdx;
			#endif
		}
	}
	
	/// Write results back to DDR (via LDS)
	UNROLL
	for (i = 0; i < 4; ++i)
	{
		const int LDSWriteOffsetY = 4 * GroupThreadID.y + i;
		const int LDSWriteOffsetX = (LDS_PADX + 4) * GroupThreadID.x;
		LDSBuffer[LDSWriteOffsetY * LDS_X_PADDEDSIZE + LDSWriteOffsetX + 0] = Values[i][0];
		LDSBuffer[LDSWriteOffsetY * LDS_X_PADDEDSIZE + LDSWriteOffsetX + 1] = Values[i][1];
		LDSBuffer[LDSWriteOffsetY * LDS_X_PADDEDSIZE + LDSWriteOffsetX + 2] = Values[i][2];
		LDSBuffer[LDSWriteOffsetY * LDS_X_PADDEDSIZE + LDSWriteOffsetX + 3] = Values[i][3];
	}

	GroupMemoryBarrierWithGroupSync();

	const int WriteOutChannelOffset = GroupThreadIdx >= 32 ? LDS_IC : 0;
	const int WriteOutPixelOffset = GroupThreadIdx % 32;
	const int LDSPixelOffset = (4 + LDS_PADX) * (WriteOutPixelOffset / 4) + WriteOutPixelOffset % 4;
	
	UNROLL
	for (i = 0; i < LDS_IC; ++i)
	{
		const int WriteOutChannel = Scalar_GroupOutputChannelOffset + i + WriteOutChannelOffset;
		if (WriteOutChannel < Cw)
		{
			const int WriteOffset = Scalar_BatchOutputOffset + WriteOutChannel * Ho * Wo + Scalar_GroupOutputPixelOffset + WriteOutPixelOffset;
			Output[WriteOffset] = LDSBuffer[(i + WriteOutChannelOffset) * LDS_X_PADDEDSIZE + LDSPixelOffset];
		}
	}
}
