// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#if DATA_TYPE == 0

#define WORK_TYPE float
#define BUFFER_TYPE float
#define BUFFER_TO_WORK_TYPE(x) x
#define WORK_TO_BUFFER_TYPE(x) x

#elif DATA_TYPE == 1

#define WORK_TYPE int
#define BUFFER_TYPE int
#define BUFFER_TO_WORK_TYPE(x) x
#define WORK_TO_BUFFER_TYPE(x) x

#elif DATA_TYPE == 2

#define WORK_TYPE uint
#define BUFFER_TYPE uint
#define BUFFER_TO_WORK_TYPE(x) x
#define WORK_TO_BUFFER_TYPE(x) x

#elif DATA_TYPE == 3

#define WORK_TYPE float
#define BUFFER_TYPE min16uint
#define BUFFER_TO_WORK_TYPE(x) f16tof32(x)
#define WORK_TO_BUFFER_TYPE(x) f32tof16(x)

#endif

// Must correspond to EConvGroupSize defined in NNEHlslShadersConvCS.h
#if GROUP_SIZE == 0
#define NUM_GROUP_THREADS 128
#elif GROUP_SIZE == 1
#define NUM_GROUP_THREADS 256
#elif GROUP_SIZE == 2
#define NUM_GROUP_THREADS 512
#endif

Buffer<BUFFER_TYPE> X; // N x C x XD0 x ... x XDi
Buffer<BUFFER_TYPE> W; // M x C/GROUPS x WD0 x ... x WDi
RWBuffer<BUFFER_TYPE> Y;  // N x M x YD0 x ... x YDi
Buffer<BUFFER_TYPE> B; // B0, ..., BM

// x: Dilation0, ...., Dilationi
// y: Strides0, ...., Stridesi
// z: Pad0Begin, ...., PadiBegin
int4 Dilation_Stride_XBlockStartOffset_DilationXBlockStride[NUM_STACK_DIMENSIONS];

// x: Element j contains PROD(GDj+1,..,GDi) with the last element containing 1, with GDi being the number of thread groups contained by a dimension i
// y: Number of threads in each dimension of a group
// z: Element j contains PROD(GTDj+1,..,GTDi) with the last element containing 1, with GTDi being the number of threads contained by a dimension i inside a group
int4 GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[NUM_STACK_DIMENSIONS];

// x: YD0, ..., YDi
// y: Element j contains PROD(YDj+1,..,YDi) with the last element containing 1
// z: XD0, ..., XDi
// w: Element j contains PROD(XDj+1,..,XDi) with the last element containing 1
int4 YDimension_YMemoryStride_XDimension_XMemoryStride[NUM_STACK_DIMENSIONS];

// x: Number of elements in each dimension of a X block
// y: The strides of the X block to be loaded
// z: WD0, ..., WDi
int4 XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[NUM_STACK_DIMENSIONS];

// x: 1/GroupStride
// y: 1/GroupThreadStride
// z: 1/XBlockStride
float4 OneDiv_GroupStride_GroupThreadStride_OneDivStride[NUM_STACK_DIMENSIONS];

int NumWChannels; // WShape[0]
int NumOutChannelsDivGroup; // WShape[1]

int YBatchStride;
int YOutputKernelStride;

int XBatchStride; // The number of elements in each bacth of X
int XChannelStride; // The number of elements in each channel of X

int XBlockSize; // The total number of elements in each block to be loaded

int NumChannelBatches; // The number of iteration needed to cover all channels that needs to be processed. Equals to ceil(NumWChannels/NumChannelsPerBatch)
int NumChannelsPerBatch; // The number of complete kernel channels that can be loaded in one batch

int WOutputKernelStride; // The total number of W elements per output kernel and thus PROD(C/GROUPS, WD0, .., WDi)
int WChannelBatchSize; // The total number of W elements inside a channel batch and thus the number of channels per batch times PROD(WD0, .., WDi)
int WChannelSize; // Number of elements in a single channel of W and thus PROD(WDi)

float GroupsDivM; // The number of groups divided by the number of output kernels used to compute the start channel offset
float OneDivGroup;

groupshared WORK_TYPE SharedMemoryX[NUM_GROUP_THREADS<<NUM_READS_PER_THREAD_POW2];
groupshared WORK_TYPE SharedMemoryW[NUM_GROUP_THREADS];

// Must correspond to EConvAlgorithm defined in NNEHlslShadersConvCS.h
#if ALGORITHM < 1

[numthreads(NUM_GROUP_THREADS, 1, 1)]
void ConvTranspose(in const int3 DispatchThreadID : SV_DispatchThreadID, in const int3 GroupID : SV_GroupID, in const int3 GroupThreadID : SV_GroupThreadID)
{
	// Group Z dimension and Y dimension are used for batch and output kernel
    // This works as each group only works on the inner dimensions (num threads in y and z set to one, group count to N and M respectively)
	int BatchIndex = GroupID.z;
	int OutputKernelIndex = GroupID.y;

	// Input channels and output kernels are divided into groups
	// Thus the channel offset for this thread's output kernel can be computed as follows:
	// The output kernel index is divided by the number of output kernels per group giving the group index
	// This is multiplied by the number of channels in W to get the start index
	int InputChannelStartIndex = (int)(GroupsDivM * (float)OutputKernelIndex);

	// Compute the output memory index and dimension indices of this thread inside its group as well as the start dimension indices of the X block to load
	int YIndex = BatchIndex * YBatchStride + OutputKernelIndex * YOutputKernelStride;
	int GroupThreadDimensionIndex[NUM_STACK_DIMENSIONS];// This will be used later to lookup the shared W and X to compute the final result
    int XBlockStartDimensionIndex[NUM_STACK_DIMENSIONS];
    float ResidualXIndex[NUM_STACK_DIMENSIONS];
	int GroupIndex = GroupID.x;
	int GroupThreadIndex = GroupThreadID.x;
	UNROLL
	for (int i = 0; i < NUM_STACK_DIMENSIONS; i++)
	{
        // Divide by the number of groups contained by one index of this dimension to get the dimension index n being the nth group in this dimension
		int GroupDimensionIndex = (int)(OneDiv_GroupStride_GroupThreadStride_OneDivStride[i].x *(float)GroupIndex);
		GroupIndex -= GroupDimensionIndex * GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[i].x;

        // Same here with number of threads contained inside the group giving the nth thread inside the group in this dimension
		GroupThreadDimensionIndex[i] = OneDiv_GroupStride_GroupThreadStride_OneDivStride[i].y * (float)GroupThreadIndex;
		GroupThreadIndex -= GroupThreadDimensionIndex[i] * (float)GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[i].z;

        float FXBlockStartDimensionIndex = ((float)GroupDimensionIndex * (float)XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[i].x + (float)Dilation_Stride_XBlockStartOffset_DilationXBlockStride[i].z) * OneDiv_GroupStride_GroupThreadStride_OneDivStride[i].w;
        ResidualXIndex[i] = (float)(FXBlockStartDimensionIndex - int(FXBlockStartDimensionIndex)) * (float)Dilation_Stride_XBlockStartOffset_DilationXBlockStride[i].y;

        // Compose the global index by the group index times group shape plus thread index inside the group
		int ThreadDimensionIndex = GroupDimensionIndex * GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[i].y + (int)GroupThreadDimensionIndex[i];

        // Subtract the padding from the index of the first element of the group to get the start index of the block needed by this group
        XBlockStartDimensionIndex[i] = (int)FXBlockStartDimensionIndex;

        // ThreadDimensionIndex is always positive but needs to be tested if it does not overflow
		if (ThreadDimensionIndex >= YDimension_YMemoryStride_XDimension_XMemoryStride[i].x || YIndex < 0)
		{
			YIndex = -1;
		}
		else
		{
			YIndex += ThreadDimensionIndex * YDimension_YMemoryStride_XDimension_XMemoryStride[i].y;
		}
	}

    // Compute the memory indices in global X which the current thread has to load as well as the indices in shared X where to write
	int GlobalXIndex[0x1<<NUM_READS_PER_THREAD_POW2];
	int SharedXIndex[0x1<<NUM_READS_PER_THREAD_POW2];
	int ReadIdx = GroupThreadID.x;
    UNROLL
    for (int j = 0; j < (int)(0x1<<NUM_READS_PER_THREAD_POW2); j++)
	{
        // If the read index exceeded the number of elements in the X block further computation can be skipped
        if (ReadIdx >= XBlockSize)
        {
            // Assign -1 for both to indicate that the element is not used and thus neither has to be read from global nor written to shared memory
            GlobalXIndex[j] = -1;
            SharedXIndex[j] = -1;
            continue;
        }

        // Turn the flat group index multiple into thread dimension indices inside the volume to be loaded by this group
        int TmpGlobalXIndex = BatchIndex * XBatchStride + InputChannelStartIndex * XChannelStride;
        int TmpSharedXIndex = 0;
        int TempReadIdx = ReadIdx;
        UNROLL
        for (int k = 0; k < NUM_STACK_DIMENSIONS; k++)
        {
            int DimensionIndex = (int)(OneDiv_GroupStride_GroupThreadStride_OneDivStride[k].z * (float)TempReadIdx);
            int DimensionIndexOffset = DimensionIndex * XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[k].y;
		    TempReadIdx -= DimensionIndexOffset;

            // Update TmpGlobalXIndex only if positive (it is already outside the volume if negative)
            if (TmpGlobalXIndex >= 0)
            {
                // Check if the global X index is in range
                int XIndex = XBlockStartDimensionIndex[k] + DimensionIndex;
                if (XIndex < 0 || XIndex >= YDimension_YMemoryStride_XDimension_XMemoryStride[k].z)
                {
                    TmpGlobalXIndex = -1;
                }
                else
                {
                    TmpGlobalXIndex += XIndex * YDimension_YMemoryStride_XDimension_XMemoryStride[k].w;
                }
            }

            // Update TmpSharedXIndex in any case as a zero has to be written
            TmpSharedXIndex += DimensionIndexOffset;
        }

        // Assign the result, which will be -1 for the global and positive for the shared index if the element lies in the padding area
        // This indicates that the global element should not be loaded but a zero has to be written to shared memory
        GlobalXIndex[j] = TmpGlobalXIndex;
        SharedXIndex[j] = TmpSharedXIndex;

        // Increment the read idx by the number of threads inside the group to cover all elements
        ReadIdx += NUM_GROUP_THREADS;
    }
 
    // Write the zeros once to avoid rewriting the same value later
    UNROLL
    for (int l = 0; l < (int)(0x1<<NUM_READS_PER_THREAD_POW2); l++)
	{
        // If the global index is out of bound (padding area) but the shared index is valid (inside the X block) the shared element needs initialization to zero
        if (GlobalXIndex[l] < 0 && SharedXIndex[l] >= 0)
        {
            SharedMemoryX[SharedXIndex[l]] = 0;
        }
    }

    // Iterate over all channel batches
    int WChannelBatchOffset = 0;
    int WChannelIndex = 0;
#if HAS_B == 0
    WORK_TYPE Result = 0;
#else
    WORK_TYPE Result = BUFFER_TO_WORK_TYPE(B[OutputKernelIndex]);
#endif
    // Each channel batch iterates over all consecutive elements of the loaded kernel channels
    // This results in bank conflict free lookups, as all threads of the same warp read exactly the same element which is then broadcasted
    for (int m = 0; m < NumChannelBatches; m++)
	{
        // Load all channels of W belonging to this batch
        int SharedWIndex = 0;
        WChannelBatchOffset = m * WChannelBatchSize;

        // Iterate over all channels in this batch
        for (int n = 0; n < NumChannelsPerBatch; n++)
	    {
            int WChannelOutputKernelOffset = WChannelSize * OutputKernelIndex + NumOutChannelsDivGroup * int((float)n * OneDivGroup);
            int WIndex = WChannelBatchOffset + GroupThreadID.x;
            if (GroupThreadID.x < WChannelBatchSize && WIndex < WOutputKernelStride)
            {  
                SharedMemoryW[GroupThreadID.x] = BUFFER_TO_WORK_TYPE(W[WChannelOutputKernelOffset + WIndex]);

            }
            if (WChannelOutputKernelOffset < WChannelSize * NumOutChannelsDivGroup * n ||
                WChannelOutputKernelOffset >= NumOutChannelsDivGroup * WChannelSize * (n + 1)) {
                SharedMemoryW[GroupThreadID.x] = 0;
            }

            // Check if the channel index is valid as channel batches may not be aligned with the effective number of channels
            if (WChannelIndex >= NumWChannels)
            {
                break;
            }
            WChannelIndex++;

            // Load the shared X
            UNROLL
            for (int o = 0; o < (int)(0x1<<NUM_READS_PER_THREAD_POW2); o++)
	        {
                // If the global index is negative, a zero has already been written, if shared index is negative there is no need to load anything
                if (GlobalXIndex[o] >= 0 && SharedXIndex[o] >= 0)
                {
                    // Load the global data into shared memory
                    SharedMemoryX[SharedXIndex[o]] = BUFFER_TO_WORK_TYPE(X[GlobalXIndex[o]]);

                    // Update the indices for the next iteration by adding the channel stride
                    GlobalXIndex[o] += XChannelStride;
                }
            }

            // Sync the group before doing any computation
            GroupMemoryBarrierWithGroupSync();

            // Initialize a dimension index array used to iterate through the kernel dimensions
            // Also initialize the start index in shared memory based on the result position
            int WDimensionIndex[NUM_STACK_DIMENSIONS];
            float SharedXIndex = 0;

            UNROLL
            for (int p = 0; p < NUM_STACK_DIMENSIONS; p++)
            {
                WDimensionIndex[p] = 0;

                // The index of the first element (e.g. upper left) required by each output is offsetted by the kernel stride and then multiplied by the stride in memory
                SharedXIndex += (GroupThreadDimensionIndex[p] + ResidualXIndex[p]) * OneDiv_GroupStride_GroupThreadStride_OneDivStride[p].w * (float)GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[p].w;
                }

            // Compute and accumulate the result
            for (int q = 0; q < WChannelSize; q++)
            {
                if (SharedXIndex - int(SharedXIndex+0.0001) < 0.001 && SharedXIndex > -0.0001){
                    Result += SharedMemoryX[SharedXIndex+0.0001] * SharedMemoryW[WChannelSize - 1 - q];
                }

                // Advance the inner-most dimension
                SharedWIndex++;
                SharedXIndex += Dilation_Stride_XBlockStartOffset_DilationXBlockStride[NUM_STACK_DIMENSIONS-1].x * OneDiv_GroupStride_GroupThreadStride_OneDivStride[NUM_STACK_DIMENSIONS-1].w;

                WDimensionIndex[NUM_STACK_DIMENSIONS-1]++;

                // Advance the other dimensions if necessary
                UNROLL
                for (int r = NUM_STACK_DIMENSIONS-1; r > 0; r--)
                {
                    if(WDimensionIndex[r] >= XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[r].z)
                    {
                        // Adjust the SharedXIndex by resetting the current dimension and increase the next outer one
                        SharedXIndex -= XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[r].w * OneDiv_GroupStride_GroupThreadStride_OneDivStride[r].w;
                        SharedXIndex += Dilation_Stride_XBlockStartOffset_DilationXBlockStride[r-1].w * OneDiv_GroupStride_GroupThreadStride_OneDivStride[r-1].w;
                        WDimensionIndex[r] = 0;
                        WDimensionIndex[r-1]++;
                    }
                    else
                    {
                        break;
                    }
                }
            }

            // Sync the group before continuing any load
            GroupMemoryBarrierWithGroupSync();
        }
    }

    // Write the final result
    if (YIndex >= 0)
    {
        Y[YIndex] = WORK_TO_BUFFER_TYPE(Result);
    }
}

#endif
