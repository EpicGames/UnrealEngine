// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input; // Ni x C x H x W
RWBuffer<float> Output; // Ni x 36 x C x Ceil(H/4)*Ceil(W/4)

int C;
int H;
int W;
int WBlockCount;

int CInputStride;
int HInputStride;

int NiOutputStride;
int MatrixOutputStride;
int COutputStride;
int HOutputStride;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ConvWinogradInput(
	in const uint3 GroupID : SV_GroupID,
	in const uint3 DispatchThreadID : SV_DispatchThreadID,
	in const uint3 GroupThreadID : SV_GroupThreadID)
{
	const int WBlock = DispatchThreadID.x;
	if (WBlock < WBlockCount)
	{
		const int WBase = WBlock * 4 - 1;
		const int Scalar_HBase = GroupID.y * 4 - 1;

		const int Scalar_InputBase = GroupID.z * CInputStride;

		int Scalar_OutputC, Scalar_OutputNi;
		DivMod(GroupID.z, C, Scalar_OutputNi, Scalar_OutputC);
		const int Scalar_OutputBase = Scalar_OutputNi * NiOutputStride + Scalar_OutputC * COutputStride + GroupID.y * HOutputStride;
		const int OutputOffset = Scalar_OutputBase + WBlock;

		// Load d
		float d[6][6];
		UNROLL
		for (int HOffset = 0; HOffset < 6; HOffset++)
		{
			const int h = Scalar_HBase + HOffset;
			const bool ValidH = h >= 0 && h < H;
			const int HInput = Scalar_InputBase + clamp(h, 0, H - 1) * HInputStride;

			UNROLL
			for (int WOffset = 0; WOffset < 6; WOffset++)
			{
				int w = WBase + WOffset;
				const bool ValidInput = ValidH && w >= 0 && w < W;
				w = clamp(w, 0, W - 1);
				const float Value = Input[HInput + w];
				d[HOffset][WOffset] = ValidInput ? Value : 0;
			}
		}
		// Calculate B^TdB
		float BTd[6][6];
		UNROLL
		for (int i = 0; i < 6; i++)
		{
			BTd[0][i] =  d[0][i] * 4 - d[2][i] * 5 + d[4][i];
			BTd[1][i] = -d[1][i] * 4 - d[2][i] * 4 + d[3][i] + d[4][i];
			BTd[2][i] =  d[1][i] * 4 - d[2][i] * 4 - d[3][i] + d[4][i];
			BTd[3][i] = -d[1][i] * 2 - d[2][i] + d[3][i] * 2 + d[4][i];
			BTd[4][i] =  d[1][i] * 2 - d[2][i] - d[3][i] * 2 + d[4][i];
			BTd[5][i] =  d[1][i] * 4 - d[3][i] * 5 + d[5][i];
		}
		float BTdB[6][6];
		UNROLL
		for (int i = 0; i < 6; i++)
		{
			BTdB[i][0] =  BTd[i][0] * 4 - BTd[i][2] * 5 + BTd[i][4];
			BTdB[i][1] = -BTd[i][1] * 4 - BTd[i][2] * 4 + BTd[i][3] + BTd[i][4];
			BTdB[i][2] =  BTd[i][1] * 4 - BTd[i][2] * 4 - BTd[i][3] + BTd[i][4];
			BTdB[i][3] = -BTd[i][1] * 2 - BTd[i][2] + BTd[i][3] * 2 + BTd[i][4];
			BTdB[i][4] =  BTd[i][1] * 2 - BTd[i][2] - BTd[i][3] * 2 + BTd[i][4];
			BTdB[i][5] =  BTd[i][1] * 4 - BTd[i][3] * 5 + BTd[i][5];
		}

		// Store result to Output
		UNROLL
		for (int j = 0; j < 6; j++)
		{
			UNROLL
			for (int i = 0; i < 6; i++)
			{
				Output[(j * 6 + i) * MatrixOutputStride + OutputOffset] = BTdB[j][i];
			}
		}
	}
}
