// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define K_STEP 16
#define BLOCK_ELEM_COUNT_M 64
#define VECTOR_SIZE 2

#define BLOCK_VECTOR_ELEM_COUNT_M (BLOCK_ELEM_COUNT_M / VECTOR_SIZE)
#define BLOCK_VECTOR_ELEM_COUNT_N (BLOCK_ELEM_COUNT_N / VECTOR_SIZE)

#define BLOCK_COUNT_M (BLOCK_ELEM_COUNT_M / 4)
#define BLOCK_COUNT_N (BLOCK_ELEM_COUNT_N / 4)
#define BLOCK_COUNT   (BLOCK_COUNT_M * BLOCK_COUNT_N)

#define THREADGROUP_SIZE_X BLOCK_VECTOR_ELEM_COUNT_M
#define THREADGROUP_SIZE_Y (BLOCK_COUNT / THREADGROUP_SIZE_X)
#define THREADGROUP_SIZE (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)

// Winograd does not compile on D3D12 SM6 + bindless if using vector<x,y> atm
// hence the define is directly set to the vectorized type in ModifyCompilationEnvironment()
// rather than defined as a templated vector here
//#define WORK_TYPE_VECTOR vector <WORK_TYPE, VECTOR_SIZE>

Buffer<WORK_TYPE_VECTOR> Input; // Ni x 36 x K x M
Buffer<WORK_TYPE_VECTOR> Weight; // 36 x K x N 
RWBuffer<WORK_TYPE_VECTOR> Output; // Ni x 36 x N x M

groupshared WORK_TYPE_VECTOR LDS_InputOutput[max(K_STEP, BLOCK_ELEM_COUNT_N)][BLOCK_VECTOR_ELEM_COUNT_M];
groupshared WORK_TYPE_VECTOR LDS_Weight[K_STEP][BLOCK_VECTOR_ELEM_COUNT_N];

int M;
int N;
int K;

int MatrixInputStride;
int KInputStride;

int MatrixWeightStride;
int KWeightStride;

int MatrixOutputStride;
int NOutputStride;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void ConvWinogradMMM(
	in const uint3 DispatchThreadID : SV_DispatchThreadID,
	in const uint3 GroupID : SV_GroupID,
	in const uint3 GroupThreadID : SV_GroupThreadID,
	in const uint GroupIndex : SV_GroupIndex)
{
	const int Scalar_NBase = GroupID.y * BLOCK_VECTOR_ELEM_COUNT_N;
	const int Scalar_InputMatrix = GroupID.z * MatrixInputStride;
	const int Scalar_WeightMatrix = (GroupID.z % 36) * MatrixWeightStride;
	const int Scalar_OutputMatrix = GroupID.z * MatrixOutputStride;
	const int Scalar_InputIndexMax = Scalar_InputMatrix + MatrixInputStride - 1;
	const int Scalar_WeightIndexMax = Scalar_WeightMatrix + MatrixWeightStride - 1;
	const int Scalar_OutputIndexMax = Scalar_OutputMatrix + MatrixOutputStride - 1;

	const int MBlock = GroupIndex % BLOCK_COUNT_M;
	const int MBlockBase = MBlock * (4 / VECTOR_SIZE);
	const int NBlock = GroupIndex / BLOCK_COUNT_M;
	const int NBlockBase = NBlock  * (4 / VECTOR_SIZE);

	matrix < WORK_TYPE, 4, 4 > Values = 0;

	const int MInputIndex = DispatchThreadID.x;
	int InputIndex = Scalar_InputMatrix;
	InputIndex += GroupThreadID.y * KInputStride;
	InputIndex += MInputIndex < M ? MInputIndex : MatrixInputStride;

	const int NWeightIndex = Scalar_NBase + GroupIndex % BLOCK_VECTOR_ELEM_COUNT_N;
	int WeightIndex = Scalar_WeightMatrix;
	WeightIndex += (GroupIndex / BLOCK_VECTOR_ELEM_COUNT_N) * KWeightStride;
	WeightIndex += NWeightIndex < N ? NWeightIndex : MatrixWeightStride;

	for (int KBase = 0; KBase < K; KBase += K_STEP)
	{
		//Load Input and Weights
		UNROLL
		for (int KOffset = 0; KOffset < K_STEP; KOffset += THREADGROUP_SIZE_Y)
		{
			const int InputIndexClamped = min(InputIndex, Scalar_InputIndexMax);
			WORK_TYPE_VECTOR InputValue = Input[InputIndexClamped];
			InputValue = InputIndex <= Scalar_InputIndexMax ? InputValue : 0;
			LDS_InputOutput[KOffset + GroupThreadID.y][GroupThreadID.x] = InputValue;

			InputIndex += THREADGROUP_SIZE_Y * KInputStride;
		}
		UNROLL
		for (int KOffset = 0; KOffset < K_STEP; KOffset += (THREADGROUP_SIZE / BLOCK_VECTOR_ELEM_COUNT_N))
		{
			const int WeightIndexClamped = min(WeightIndex, Scalar_WeightIndexMax);
			WORK_TYPE_VECTOR WeightValue = Weight[WeightIndexClamped];
			WeightValue = WeightIndex <= Scalar_WeightIndexMax ? WeightValue : 0;
			LDS_Weight[KOffset + GroupIndex / BLOCK_VECTOR_ELEM_COUNT_N][GroupIndex % BLOCK_VECTOR_ELEM_COUNT_N] = WeightValue;

			WeightIndex += (THREADGROUP_SIZE / BLOCK_VECTOR_ELEM_COUNT_N) * KWeightStride;
		}

		GroupMemoryBarrierWithGroupSync();

		ISOLATE
		{
			//Multiply
			UNROLL
			for (int KIndex = 0; KIndex < K_STEP; KIndex++)
			{
				matrix < WORK_TYPE, 1, 4 > Reg_Input = { LDS_InputOutput[KIndex][MBlockBase], LDS_InputOutput[KIndex][MBlockBase + 1] };
				matrix < WORK_TYPE, 4, 1 > Reg_Weight = { LDS_Weight[KIndex][NBlockBase], LDS_Weight[KIndex][NBlockBase + 1] };

				Values += mul(Reg_Weight, Reg_Input);
			}
		}

		GroupMemoryBarrierWithGroupSync();
	}

	// Move values from Reg to LDS
	UNROLL
	for (int NOffset = 0; NOffset < 4; NOffset++)
	{
		const int NLDSIndex = NBlock * 4 + NOffset;
		LDS_InputOutput[NLDSIndex][MBlockBase + 0] = Values[NOffset].xy;
		LDS_InputOutput[NLDSIndex][MBlockBase + 1] = Values[NOffset].zw;
	}

	
	GroupMemoryBarrierWithGroupSync();

	// Move values from LDS to VRAM
	const int MOutputIndex = DispatchThreadID.x;
	int OutputIndex = Scalar_OutputMatrix;
	OutputIndex += Scalar_NBase * VECTOR_SIZE * NOutputStride;
	OutputIndex += GroupThreadID.y * NOutputStride;
	OutputIndex += MOutputIndex < M ? MOutputIndex : MatrixOutputStride;

	UNROLL
	for (int NOffset = 0; NOffset < BLOCK_ELEM_COUNT_N; NOffset += THREADGROUP_SIZE_Y)
	{
		if (OutputIndex <= Scalar_OutputIndexMax)
		{
			Output[OutputIndex] = LDS_InputOutput[NOffset + GroupThreadID.y][GroupThreadID.x];
		}
		OutputIndex += THREADGROUP_SIZE_Y * NOutputStride;
	}
}
