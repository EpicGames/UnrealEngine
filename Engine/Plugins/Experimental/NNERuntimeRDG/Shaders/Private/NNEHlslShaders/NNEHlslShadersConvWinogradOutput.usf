// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<WORK_TYPE> Input; // Ni x 36 x CExtended x Ceil(H/4)*Ceil(W/4)
Buffer<WORK_TYPE> Bias; // C
RWBuffer<WORK_TYPE> Output; // Ni x C x H x W

int C;
int H;
int W;
int WBlockCount;

int NiInputStride;
int MatrixInputStride;
int CInputStride;
int HInputStride;

int COutputStride;
int HOutputStride;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ConvWinogradOutput(
	in const uint3 DispatchThreadID : SV_DispatchThreadID,
	in const uint3 GroupID : SV_GroupID,
	in const uint3 GroupThreadID : SV_GroupThreadID)
{
	const int WBlock = DispatchThreadID.x;
	if (WBlock < WBlockCount)
	{
		int Scalar_C, Scalar_Ni;
		DivMod(GroupID.z, C, Scalar_Ni, Scalar_C);

		const int InputOffset = Scalar_Ni * NiInputStride + Scalar_C * CInputStride + GroupID.y * HInputStride + WBlock;
		const int Scalar_OutputBase = GroupID.z * COutputStride + (GroupID.y * HOutputStride + WBlock) * 4;

		// Load block from Input
		WORK_TYPE I[6][6];
		UNROLL
		for (int j = 0; j < 6; j++)
		{
			UNROLL
			for (int i = 0; i < 6; i++)
			{
				I[j][i] = Input[(j * 6 + i) * MatrixInputStride + InputOffset];
			}
		}

		// Calculate A^TIA
		WORK_TYPE ATI[4][6];
		UNROLL
		for (int i = 0; i < 6; i++)
		{
			ATI[0][i] = I[0][i] + I[1][i] + I[2][i] + I[3][i] + I[4][i];
			ATI[1][i] = I[1][i] - I[2][i] + I[3][i] * 2 - I[4][i] * 2;
			ATI[2][i] = I[1][i] + I[2][i] + I[3][i] * 4 + I[4][i] * 4;
			ATI[3][i] = I[1][i] - I[2][i] + I[3][i] * 8 - I[4][i] * 8 + I[5][i];
		}
		WORK_TYPE ATIA[4][4];
		UNROLL
		for (int i = 0; i < 4; i++)
		{
			ATIA[i][0] = ATI[i][0] + ATI[i][1] + ATI[i][2] + ATI[i][3] + ATI[i][4];
			ATIA[i][1] = ATI[i][1] - ATI[i][2] + ATI[i][3] * 2 - ATI[i][4] * 2;
			ATIA[i][2] = ATI[i][1] + ATI[i][2] + ATI[i][3] * 4 + ATI[i][4] * 4;
			ATIA[i][3] = ATI[i][1] - ATI[i][2] + ATI[i][3] * 8 - ATI[i][4] * 8 + ATI[i][5];
		}

#if HAS_BIAS
		WORK_TYPE Scalar_BiasValue = Bias[Scalar_C];
#endif

		// Store result
		for (int HOffset = 0; HOffset < 4; HOffset++)
		{
			const int HIndex = GroupID.y * 4 + HOffset;
			if(HIndex < H)
			{
				for (int WOffset = 0; WOffset < 4; WOffset++)
				{
					const int WIndex = WBlock * 4 + WOffset;
					if (WIndex < W)
					{
#if HAS_BIAS
						Output[Scalar_OutputBase + HOffset * HOutputStride + WOffset] = ATIA[HOffset][WOffset] + Scalar_BiasValue;
#else
						Output[Scalar_OutputBase + HOffset * HOutputStride + WOffset] = ATIA[HOffset][WOffset];
#endif
					}
				}
			}
		}
	}
}
