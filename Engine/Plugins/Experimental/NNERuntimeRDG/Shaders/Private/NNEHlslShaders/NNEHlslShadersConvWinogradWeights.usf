// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define KERNEL_STRIDE 9

Buffer<float> Input; // Cw x Ci x 3 x 3
RWBuffer<float> Output; // 36 x Ci x Cw

int Ci;
int Cw;

int CwInputStride;

int MatrixOutputStride;
int CiOutputStride;


[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ConvWinogradWeights(
	in const uint3 GroupID : SV_GroupID,
	in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const int ci = DispatchThreadID.x;
	if (ci < Ci)
	{
		const int Scalar_Cw      = GroupID.y;
		const int Input_Base 	= (Scalar_Cw * CwInputStride + ci) * KERNEL_STRIDE;
		const int Output_Offset = ci * CiOutputStride + Scalar_Cw;

		// Loading g
		float g[3][3];
		UNROLL
		for (int j = 0; j < 3; j++)
		{			
			UNROLL
			for (int i = 0; i < 3; i++)
			{
				g[j][i] = Input[Input_Base + j * 3 + i];
			}
		}

		//Calculating GgG^T
		float Gg[6][3];
		UNROLL
		for (int i = 0; i < 3; i++)
		{
			Gg[0][i] = g[0][i]/4;
			Gg[1][i] = (-g[0][i] - g[1][i] - g[2][i]) / 6;
			Gg[2][i] = (-g[0][i] + g[1][i] - g[2][i]) / 6;
			Gg[3][i] = (g[0][i] + g[1][i]*2 + g[2][i]*4) / 24;
			Gg[4][i] = (g[0][i] - g[1][i]*2 + g[2][i]*4) / 24;
			Gg[5][i] = g[2][i];
		}
		float GgGT[6][6];
		UNROLL
		for (int i = 0; i < 6; i++)
		{
			GgGT[i][0] = Gg[i][0]/4;
			GgGT[i][1] = (-Gg[i][0] - Gg[i][1] - Gg[i][2]) / 6;
			GgGT[i][2] = (-Gg[i][0] + Gg[i][1] - Gg[i][2]) / 6;
			GgGT[i][3] = (Gg[i][0] + Gg[i][1]*2 + Gg[i][2]*4) / 24;
			GgGT[i][4] = (Gg[i][0] - Gg[i][1]*2 + Gg[i][2]*4) / 24;
			GgGT[i][5] = Gg[i][2];
		}

		// Store result
		UNROLL
		for (int j = 0; j < 6; j++)
		{
			UNROLL
			for (int i = 0; i < 6; i++)
			{
				Output[(j * 6 + i) * MatrixOutputStride + Output_Offset] = GgGT[j][i];
			}
		}
	}
}
