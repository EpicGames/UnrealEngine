// Copyright Epic Games, Inc. All Rights Reserved.
//
// Implementation based on Single-pass Parallel Prefix Scan with Decoupled Look-back
// Original research paper: https://research.nvidia.com/publication/2016-03_single-pass-parallel-prefix-scan-decoupled-look-back

#include "/Engine/Public/Platform.ush"


#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

uint NumThreadGroupsPerScan; // Each group's x-axis refers to a single prefix scan (cumulative sum)
uint NumThreadGroupsY; // Each group's y-index refers to pre-axis dimensions
uint NumThreadGroupsZ; // Each group's z-index refers to post-axis dimensions

#define STATUS_INVALID				0
#define STATUS_AGGREGATE_AVAILABLE  1
#define STATUS_PREFIX_AVAILABLE  	2

/*
	For each scan there is a different global partition index.
	Each group in the same scan lock-step increments the global partition index. 
	The value before the increment is the partition index assigned to the thread group that did the increment.
*/
globallycoherent RWStructuredBuffer<uint> GlobalPartitionIndex;
/*  
	Elements to scan are partitioned. Each thread group computes the prefix scan of a partition. 
	Results to be propagated to the next groups are written in the partition descriptor in device memory.
	It needs to be `globallycoherent` so that memory fences flush updates across groups.
	There are as many descriptors as thread groups.
*/
struct FPartitionDescriptor
{
	int StatusFlag;
	WORK_TYPE Aggregate;
	WORK_TYPE InclusivePrefix;
	int PadToQWord;
};
globallycoherent RWStructuredBuffer<FPartitionDescriptor> PartitionDescriptor;


#ifdef INIT_SHADER

uint NumInitThreadGroups;

[numthreads(INIT_THREADGROUP_SIZE, 1, 1)]
void InitCumSum(uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Increment = INIT_THREADGROUP_SIZE * NumInitThreadGroups;

	for (uint Idx = DispatchThreadID.x; Idx < NumThreadGroupsPerScan * NumThreadGroupsY * NumThreadGroupsZ; Idx += Increment)
	{
#if METAL_SM6_PROFILE
		PartitionDescriptor[Idx].StatusFlag = STATUS_INVALID;
		PartitionDescriptor[Idx].InclusivePrefix = (WORK_TYPE) 0;
		PartitionDescriptor[Idx].Aggregate = (WORK_TYPE) 0;
		PartitionDescriptor[Idx].PadToQWord = 0;
#else
        FPartitionDescriptor InitPD;

		InitPD.StatusFlag = STATUS_INVALID;
		InitPD.InclusivePrefix = (WORK_TYPE) 0;
		InitPD.Aggregate = (WORK_TYPE) 0;
		InitPD.PadToQWord = 0;

		PartitionDescriptor[Idx] = InitPD;
#endif

		if (Idx % NumThreadGroupsPerScan == 0)
		{
			GlobalPartitionIndex[Idx / NumThreadGroupsPerScan] = 0u;
		}
	}
}

#else // !INIT_SHADER

Buffer<BUFFER_TYPE> Input;
RWBuffer<BUFFER_TYPE> Output;

#define STRIDE_IDX 0

#define MIN_WAVE_LANES 4U
#define PARTITION_SIZE (THREADGROUP_SIZE * VALUES_PER_THREAD)
#define NUM_WAVEAGGREGATES (THREADGROUP_SIZE / WaveGetLaneCount())
#define LAST_WAVEAGGREGATE_ID (NUM_WAVEAGGREGATES - 1)

uint NumScanValues;
uint Axis;
uint AxisStride;

/* Group shared memory variables */

groupshared uint GroupPartitionIndex;
groupshared WORK_TYPE GroupExclusivePrefix;
groupshared WORK_TYPE LocalPrefix[PARTITION_SIZE]; // Wave-local prefixes for each element of the partition (before adding the group's exclusive prefix)
groupshared WORK_TYPE WaveAggregate[THREADGROUP_SIZE / MIN_WAVE_LANES]; // Wave-wide aggregates (also group-local) to speed up reduction phase


inline WORK_TYPE GetScanInput(uint IdxWithinAxis, uint ScanStartGlobalIdx)
{
	return Input[ ScanStartGlobalIdx + IdxWithinAxis * AxisStride ];
}

inline void SetScanOutput(uint IdxWithinAxis, WORK_TYPE Value, uint ScanStartGlobalIdx)
{
	Output[ ScanStartGlobalIdx + IdxWithinAxis * AxisStride ] = Value;
}

inline uint GetWaveIndex(uint GroupThreadID)
{
	return GroupThreadID / WaveGetLaneCount();
}

inline uint PartitionOffset(uint PartitionIndex)
{
	return PartitionIndex * PARTITION_SIZE;
}

inline uint WavePartitionOffset(uint GroupThreadID)
{
	return GetWaveIndex(GroupThreadID) * VALUES_PER_THREAD * WaveGetLaneCount();
}

/*
	Performs a wave-local prefix scan (and reduction) of VALUES_PER_THREAD elements per thread, for each wave.
	Scan results are stored in groupshared memory array LocalPrefix.
	The aggregated value for each wave is stored in groupshared memory array WaveAggregate.
*/
inline void WaveScanReduce(uint GroupThreadID, uint PartitionIndex, uint ScanStartGlobalIdx, bool bPartial)
{
	const uint ShiftLaneIdx = WaveGetLaneIndex() != 0u ? (WaveGetLaneIndex() - 1u) : (WaveGetLaneCount() - 1u);
	const uint PartialSize = NumScanValues - PartitionOffset(PartitionIndex);
	WORK_TYPE Aggregate = 0;

	[unroll]
	for (uint Idx = WaveGetLaneIndex() + WavePartitionOffset(GroupThreadID), WordIdx = 0; WordIdx < VALUES_PER_THREAD; Idx += WaveGetLaneCount(), ++WordIdx)
	{
		WORK_TYPE Value = !bPartial || Idx < PartialSize ? 
				GetScanInput(Idx + PartitionOffset(PartitionIndex), ScanStartGlobalIdx) 
			: 	0;
		
		const WORK_TYPE ShiftLaneValue = WaveReadLaneAt(Value + WavePrefixSum(Value), ShiftLaneIdx); // Wave prefix sum and circular shuffle
		LocalPrefix[Idx] = Value + (WaveGetLaneIndex() != 0u ? ShiftLaneValue : 0) + Aggregate;
		Aggregate += WaveReadLaneAt(ShiftLaneValue, 0);
	}

	if (WaveGetLaneIndex() == 0u)
	{
		WaveAggregate[GetWaveIndex(GroupThreadID)] = Aggregate;
	}
}

/*
	Performs a group-local prefix scan of the wave aggregates in groupshared memory.
*/
inline void ScanWaveAggregate(uint GroupThreadID)
{
	const uint ScanSize = NUM_WAVEAGGREGATES;

	if (GroupThreadID < WaveGetLaneCount())
	{
		WaveAggregate[GroupThreadID] += WavePrefixSum(WaveAggregate[GroupThreadID]);

		// The following won't be needed if WaveGetLaneCount() * WaveGetLaneCount() >= THREADGROUP_SIZE (most common case).
		for(uint AggregateOffset = WaveGetLaneCount(); AggregateOffset + GroupThreadID < ScanSize; AggregateOffset += WaveGetLaneCount())
		{
			GroupMemoryBarrierWithGroupSync();
			if (WaveGetLaneIndex() == 0u)
			{
				WaveAggregate[AggregateOffset] += WaveAggregate[AggregateOffset - 1u];
			}
			GroupMemoryBarrierWithGroupSync();
			WaveAggregate[GroupThreadID + AggregateOffset] += WavePrefixSum(WaveAggregate[GroupThreadID + AggregateOffset]);

		}
	}

}


inline void ReservePartitionIndex(uint GroupThreadID, uint ScanID)
{
	if (GroupThreadID == 0)
	{
		InterlockedAdd(GlobalPartitionIndex[ScanID], 1u, GroupPartitionIndex);
	}
}

inline void SetPartitionInclusivePrefix(uint PDIdx, WORK_TYPE Value)
{
	PartitionDescriptor[PDIdx].InclusivePrefix = Value;
	DeviceMemoryBarrier(); // To guarantee coherency avoid that status gets written before inclusive prefix does
	PartitionDescriptor[PDIdx].StatusFlag = STATUS_PREFIX_AVAILABLE;
}

inline void SetPartitionAggregate(uint PDIdx, WORK_TYPE Value)
{
	PartitionDescriptor[PDIdx].Aggregate = Value;
	DeviceMemoryBarrier(); // To guarantee coherency avoid that status gets written before aggregate does
	PartitionDescriptor[PDIdx].StatusFlag = STATUS_AGGREGATE_AVAILABLE;
}

inline uint GetPDIdx(uint PartitionIndex, uint ScanID)
{
	return ScanID * NumThreadGroupsPerScan + PartitionIndex;
}
/*
	Updates the current group's partition descriptor in device memory once the partition aggregate has been computed.
*/
inline void BroadcastGroupAggregate(uint GroupThreadID, uint PartitionIndex, uint ScanID)
{
	// Last wave aggregate corresponds to group (partition) aggregate
	if (GroupThreadID == LAST_WAVEAGGREGATE_ID)
    {
		const uint PDIdx = GetPDIdx(PartitionIndex, ScanID);
		if(PartitionIndex == 0)
		{
			SetPartitionInclusivePrefix(PDIdx, WaveAggregate[GroupThreadID]);
		}
		else
		{
			SetPartitionAggregate(PDIdx, WaveAggregate[GroupThreadID]);
		}
	}
}
/*
	Computes the partition exclusive prefix by recursively summing up prefixes/aggregates of previous partitions in backward order.
	Also computes and sets (broadcasts) the inclusive prefix of the partition once the exclusive prefix is available.
*/
inline void DecoupledLookback(uint PartitionIndex, uint ScanID)
{
	WORK_TYPE PreviousReduction = (WORK_TYPE) 0;
	int LookBackIndex = (int) PartitionIndex - 1;

	while (LookBackIndex >= 0)
	{
		const FPartitionDescriptor LookBackPD = PartitionDescriptor[GetPDIdx(LookBackIndex, ScanID)];

		if (LookBackPD.StatusFlag == STATUS_PREFIX_AVAILABLE)
		{
			DeviceMemoryBarrier(); // Required between reading lookback's StatusFlag and InclusivePrefix for coherency
			PreviousReduction += LookBackPD.InclusivePrefix;
			GroupExclusivePrefix = PreviousReduction;
			SetPartitionInclusivePrefix(
				GetPDIdx(PartitionIndex, ScanID), 
				PreviousReduction + WaveAggregate[LAST_WAVEAGGREGATE_ID]
				);
			return;
		}
		else if(LookBackPD.StatusFlag == STATUS_AGGREGATE_AVAILABLE)
		{
			DeviceMemoryBarrier(); // Required between reading lookback's StatusFlag and Aggregate for coherency
			PreviousReduction += LookBackPD.Aggregate;
			LookBackIndex--;
		}

	}

	GroupExclusivePrefix = PreviousReduction;
	SetPartitionInclusivePrefix(
		GetPDIdx(PartitionIndex, ScanID), 
		PreviousReduction + WaveAggregate[LAST_WAVEAGGREGATE_ID]
		);
}
/*
	Computes the final prefix scan by adding wave-local prefixes to wave-wise exclusive prefixes.
*/
void SeededWaveScan(uint GroupThreadID, uint PartitionIndex, WORK_TYPE ExclusivePrefix, uint ScanStartGlobalIdx, bool bPartial)
{
	const uint PartialSize = NumScanValues - PartitionOffset(PartitionIndex);

	[unroll]
	for (uint Idx = WaveGetLaneIndex() + WavePartitionOffset(GroupThreadID), WordIdx = 0; WordIdx < VALUES_PER_THREAD && (!bPartial || Idx < PartialSize); Idx += WaveGetLaneCount(), ++WordIdx)
	{
		SetScanOutput(Idx + PartitionOffset(PartitionIndex), LocalPrefix[Idx] + ExclusivePrefix, ScanStartGlobalIdx);
	}

}


[numthreads(THREADGROUP_SIZE, 1, 1)]
void CumSum(in const uint3 GroupID : SV_GroupID, in const uint3 GroupThreadIDVec : SV_GroupThreadID)
{
	const uint PreAxisGroupID = GroupID.y;
	const uint PostAxisGroupID = GroupID.z;
	const uint GroupThreadID = GroupThreadIDVec.x;
	const uint ScanID = PreAxisGroupID * NumThreadGroupsZ + PostAxisGroupID;

	// Global index to first element of the scan
	uint ScanStartGlobalIdx = PreAxisGroupID * (NumScanValues * NumThreadGroupsZ) + PostAxisGroupID;

	ReservePartitionIndex(GroupThreadID, ScanID);
	GroupMemoryBarrierWithGroupSync();
	const uint PartitionIndex = GroupPartitionIndex;

	WaveScanReduce(GroupThreadID, PartitionIndex, ScanStartGlobalIdx, /* bPartial */ PartitionIndex == NumThreadGroupsPerScan - 1);
	GroupMemoryBarrierWithGroupSync();
	ScanWaveAggregate(GroupThreadID);

	BroadcastGroupAggregate(GroupThreadID, PartitionIndex, ScanID);

	GroupExclusivePrefix = 0.0f;
	
	if (PartitionIndex != 0 && GroupThreadID == 0)
	{
		DecoupledLookback(PartitionIndex, ScanID);
	}
	GroupMemoryBarrierWithGroupSync(); // Required to sync all threads in the group with respect to GroupExclusivePrefix

	// Compute the wave-wise exclusive prefix
	const WORK_TYPE ExclusivePrefix = GroupExclusivePrefix + 
			(GroupThreadID >= WaveGetLaneCount() ? WaveAggregate[GetWaveIndex(GroupThreadID) - 1] : (WORK_TYPE) 0)
		;

	SeededWaveScan(GroupThreadID, PartitionIndex, ExclusivePrefix, ScanStartGlobalIdx, /* bPartial */ PartitionIndex == NumThreadGroupsPerScan - 1);

}

#endif //ifdef INIT_SHADER