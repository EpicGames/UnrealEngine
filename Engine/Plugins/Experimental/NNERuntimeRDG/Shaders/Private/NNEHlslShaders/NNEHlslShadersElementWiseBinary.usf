// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> LHSInput;
Buffer<float> RHSInput;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;

// See https://github.com/onnx/onnx/blob/main/docs/Operators.md#prelu
float prelu(float x, float slope)
{
	return (x < 0.0f) ? (slope * x) : x;
}

float safe_pow(float base, float xp)
{
	if (base >= 0.0f)
		return pow(base, xp);
	
	//negative base + not integer exponent => nan
	if (round(xp) != xp)
	{
		
		const float nan = 0. / 0.;
		return nan;
	}

	//negative base + integer exponent
	int isExpOdd = (xp % 2);
	float sign = isExpOdd ? -1.0f : 1.0f;
	return sign * pow(-base, xp);
}

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ElementWiseBinary(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint LHSIndex = 0;
		uint RHSIndex = 0;
		uint Offset = Index;
		for (uint dim = 0; dim < NUM_DIMENSIONS; ++dim)
		{
			uint Q, R;
			DivMod(Offset, TensorInfo[dim][2], Q, R);
			LHSIndex += TensorInfo[dim][0] * Q;
			RHSIndex += TensorInfo[dim][1] * Q;
			Offset = R;
		}

		float X = LHSInput[LHSIndex];
		float Y = RHSInput[RHSIndex];
		Output[Index] = ELEMENTWISE_OP(X, Y);
	}
}
