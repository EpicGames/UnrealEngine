// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

Buffer<float> Input;
RWBuffer<float> Output;
uint Num;
uint ThreadCountX;
float Alpha;
Buffer<float> AlphaTensor;
float Beta;
Buffer<float> BetaTensor;
float Gamma;

float GetAlpha()
{
#if ALPHA_ON_GPU == 0
	return Alpha;
#else
	return AlphaTensor[0];
#endif
}

float GetBeta()
{
#if BETA_ON_GPU == 0
	return Beta;
#else
	return BetaTensor[0];
#endif
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#Relu
float relu(float x)
{
	return max(0.0f, x);
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#Selu
float selu(float x)
{
	float yNegOrZero = Gamma * (GetAlpha() * exp(x) - GetAlpha());
	float yPos = Gamma * x;
	return (x > 0.0f) ? yPos : yNegOrZero;
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#sigmoid
float sigmoid(float x)
{
	return 1.0f / (1.0f + exp(-x));
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#softplus
float softplus(float x)
{
	return log(exp(x) + 1.0f);
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#softsign
float softsign(float x)
{
	return x/(1.0f+abs(x));
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#acosh
float acosh(float x)
{
	//https://mathworld.wolfram.com/InverseHyperbolicCosine.html
	const static float NaN = 0.0f / 0.0f;
	float yAboveOne = log( x + sqrt( x + 1.0f ) + sqrt( x - 1.0f ) );
	return (x == 1.0f) ? 0.0f : (x >= 1.0f) ? yAboveOne : NaN;
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#asinh
float asinh(float x)
{
	//https://mathworld.wolfram.com/InverseHyperbolicSine.html
	return log( x + sqrt( 1 + ( x * x ) ) );
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#atanh
float atanh(float x)
{
	//https://mathworld.wolfram.com/InverseHyperbolicTangent.html
	return 0.5f * ( log( 1 + x) - log ( 1 - x) );
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#elu
float elu(float x)
{
    float yNeg = GetAlpha() * (exp(x) - 1.0f);
	float yPosOrZero = x;
	return (x >= 0.0f) ? yPosOrZero : yNeg;
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#erf
float erf(float x)
{
	// On some platforms tanh() gives NaN for large inputs, 
	// thus better directly return the asymptotic value (+1 or -1) for abs(x) >= 4
	if(abs(x) >= 4.0f)
	{
		return asfloat((asuint(x) & 0x80000000) ^ asuint(1.0f));
	}
	//https://aapt.scitation.org/doi/abs/10.1119/1.15018?journalCode=ajp
	float a = 167.0f/148.0f;
	float b = 11.0f/109.0f;
	float x3 = x*x*x;
	return tanh( a*x + b*x3 );
}

float hardSigmoid(float x, float alpha, float beta)
{
	return max(0.0f, min(1.0f, alpha * x + beta));
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#hardSigmoid
float hardSigmoid(float x)
{
	return hardSigmoid(x, GetAlpha(), GetBeta());
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#hardSwish
float hardSwish(float x)
{
    return x * hardSigmoid(x, 1.0f/6.0f, 0.5f);
}

//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#leakyRelu
float leakyRelu(float x)
{
    return (x >= 0.0f) ? x : GetAlpha() * x;
}


//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#clip
float clipOp(float x)
{
	float min = GetAlpha();
	float max = GetBeta();
	return clamp(x, GetAlpha(), GetBeta());
}


[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ElementWiseUnary(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	if (Index < Num)
	{
		Output[Index] = ELEMENTWISE_OP(Input[Index]);
	}
}
