// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

#define BUFFER_TYPE float

Buffer<BUFFER_TYPE> Input0;
Buffer<BUFFER_TYPE> Input1;
Buffer<BUFFER_TYPE> Input2;
Buffer<BUFFER_TYPE> Input3;
RWBuffer<BUFFER_TYPE> Output;
uint4 InputTensorInfo[NUM_DIMENSIONS];
uint4 OutputTensorInfo[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;
float Scale;

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void ElementWiseVariadic(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint Input0Index = 0;
		uint Input1Index = 0;
		uint Input2Index = 0;
		uint Input3Index = 0;
		uint Offset = Index;
		for (uint dim = 0; dim < NUM_DIMENSIONS; ++dim)
		{
			uint Q, R;
			DivMod(Offset, OutputTensorInfo[dim][0], Q, R);
			Input0Index += InputTensorInfo[dim][0] * Q;
			Input1Index += InputTensorInfo[dim][1] * Q;
			Input2Index += InputTensorInfo[dim][2] * Q;
			Input3Index += InputTensorInfo[dim][3] * Q;
			Offset = R;
		}

		BUFFER_TYPE Result = Input0[Input0Index];

		 // Must correspond to EVariadicNumInput defined in NNEElementWiseVariadicCS.h
		#if NUMINPUT >= 2
			Result  = ELEMENTWISE_OP(Result, Input1[Input1Index]);
		#endif
		#if NUMINPUT >= 3
			Result  = ELEMENTWISE_OP(Result, Input2[Input2Index]);
		#endif
		#if NUMINPUT >= 4
			Result  = ELEMENTWISE_OP(Result, Input3[Input3Index]);
		#endif

		#if OUTPUTASINPUT
			Result  = ELEMENTWISE_OP(Result, Output[Index]);
		#endif

		#if APPLYSCALE
			Result  = (BUFFER_TYPE)(Result * Scale);
		#endif

		Output[Index] = Result;
	}
}
