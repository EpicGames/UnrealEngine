// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersTypeHelper.ush"

int Axis;
int OutputSize;
int NumDataDimensions;
int NumIndicesDimensions;
int4 DataStride_IndicesStride_OutputStride_DataSizes[MAX_NUM_DIMENSIONS];
float4 OneDivDataStride_OneDivIndicesStride_OneDivOutputStride[MAX_NUM_DIMENSIONS];


Buffer<float> Data;
Buffer<int> Indices;
RWBuffer<float> Output;

uint NormalizeIdx(int Idx, uint DimSize)
{
	return Idx < 0 ? (uint) (Idx + DimSize) : (uint) Idx;
}

// int64_t is only available from SM 6.0, therefore here we need to manually cast to int32.
int CastInt64ToInt32(uint LSW, int MSW)
{
	return (MSW >> 31) == 1 ? - (int) LSW : (int) LSW;
}

[numthreads(NUM_GROUP_THREADS, 1, 1)]
void Gather(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x >= OutputSize)
	{
		return;
	}

	// Compute the output index per dimenion
	int OutputIndices[NUM_OUTPUT_DIMENSIONS];
	int DispatchThreadIndex = DispatchThreadID.x;
	UNROLL
	for (int i = 0; i < NUM_OUTPUT_DIMENSIONS; i++)
	{
		OutputIndices[i] = (int)(OneDivDataStride_OneDivIndicesStride_OneDivOutputStride[i].z * (float)DispatchThreadIndex);
		DispatchThreadIndex -= OutputIndices[i] * DataStride_IndicesStride_OutputStride_DataSizes[i].z;
	}

	// Split the output indices into data and indices indices 
	int DataIndices[NUM_OUTPUT_DIMENSIONS + 1];
	int IndicesIndices[NUM_OUTPUT_DIMENSIONS];
	for (int i = 0; i < Axis; i++)
	{
		DataIndices[i] = OutputIndices[i];
	}
	for (int i = 0; i < NumIndicesDimensions; i++)
	{
		IndicesIndices[i] = OutputIndices[Axis + i];
	}
	for (int i = Axis + 1; i < NumDataDimensions; i++)
	{
		DataIndices[i] = OutputIndices[NumIndicesDimensions + i - 1];
	}

	// Compute the flattened indices index
	int IndicesIndex = 0;
	for (int i = 0; i < NumIndicesDimensions; i++)
	{
		IndicesIndex += IndicesIndices[i] * DataStride_IndicesStride_OutputStride_DataSizes[i].y;
	}

	DataIndices[Axis] = NormalizeIdx(GetInt32IndexFromBuffer(Indices, IndicesIndex),
								     DataStride_IndicesStride_OutputStride_DataSizes[Axis].w);

	// Compute the flattened data index
	int DataIndex = 0;
	for (int i = 0; i < NumDataDimensions; i++)
	{
		DataIndex += DataIndices[i] * DataStride_IndicesStride_OutputStride_DataSizes[i].x;
	}

	// Write the result
	Output[DispatchThreadID.x] = Data[DataIndex];
}