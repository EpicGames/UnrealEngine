// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersTypeHelper.ush"

Buffer<float> Input;
Buffer<int> Indices;
RWBuffer<float> Output;

int Axis;
int AxisSize;
int OutputSize;
uint ThreadCountX;
float4 OneDivOutputStrides[NUM_DIMENSIONS];
int4 Input_OutputStrides[NUM_DIMENSIONS];

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void GatherElements(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;

	if(Index >= OutputSize)
	{
		return;
	}

	int OutputIndices[NUM_DIMENSIONS];
	int CurrentIndex = Index;
	UNROLL
	for (int i = 0; i < NUM_DIMENSIONS; i++)
	{
		OutputIndices[i] = (int) (OneDivOutputStrides[i].x * CurrentIndex);
		CurrentIndex -= OutputIndices[i] * Input_OutputStrides[i].y;
	}

	int AxisIndex = GetInt32IndexFromBuffer(Indices, Index);
	if(AxisIndex < 0)
	{
		AxisIndex += AxisSize;
	}
	OutputIndices[Axis] = AxisIndex;
	
	int InputIndex = 0;
	UNROLL
	for (int i = 0; i < NUM_DIMENSIONS; i++)
	{
		InputIndex += OutputIndices[i] * Input_OutputStrides[i].x;
	}

	Output[Index] = Input[InputIndex];
}
