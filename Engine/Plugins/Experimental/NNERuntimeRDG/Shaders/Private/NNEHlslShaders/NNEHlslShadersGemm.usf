// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

WORK_TYPE Alpha;
WORK_TYPE Beta;
int TransA;
int TransB;
uint M;
uint N;
uint K;
uint MxK;
uint KxN;
uint MxN;
uint CWidth;
uint CHeight;

WORK_TYPE CScalar;

Buffer<BUFFER_TYPE> A;
Buffer<BUFFER_TYPE> B;
Buffer<BUFFER_TYPE> C;
RWBuffer<BUFFER_TYPE> Y;

// x: StackShapeA[0], ..., StackShapeA[NUM_STACK_DIMENSIONS - 1]
// y: StackShapeB[0], ..., StackShapeB[NUM_STACK_DIMENSIONS - 1]
// z: StackStrideA[0], ..., StackStrideA[NUM_STACK_DIMENSIONS - 1]
// w: StackStrideB[0], ..., StackStrideB[NUM_STACK_DIMENSIONS - 1]
uint4 StackShapeA_StackShapeB_StackStrideA_StackStrideB[MAX_NUM_STACK_DIMENSIONS];

uint3 GetMatrixStackOffsets(const uint3 GroupID)
{
// Must correspond to FGemmNumDimensions defined in NNEHlslShadersGemmCS.h
#if NUM_STACK_DIMENSIONS == 0
	return uint3(0, 0, 0);
#else
	uint stackCoordA[NUM_STACK_DIMENSIONS];
	uint stackCoordB[NUM_STACK_DIMENSIONS];

	uint stackStrideOutput[NUM_STACK_DIMENSIONS];
	stackStrideOutput[NUM_STACK_DIMENSIONS - 1] = 1;
	for (int i = NUM_STACK_DIMENSIONS - 2; i >= 0; i--)
	{
		uint maxDim = max(StackShapeA_StackShapeB_StackStrideA_StackStrideB[i + 1][0], StackShapeA_StackShapeB_StackStrideA_StackStrideB[i + 1][1]);
		stackStrideOutput[i] = maxDim * stackStrideOutput[i + 1];
	}

	uint idx = GroupID.z;
	for (int i = 0; i < NUM_STACK_DIMENSIONS; i++)
	{
		uint coord = idx / stackStrideOutput[i];
		stackCoordA[i] = coord % StackShapeA_StackShapeB_StackStrideA_StackStrideB[i][0];
		stackCoordB[i] = coord % StackShapeA_StackShapeB_StackStrideA_StackStrideB[i][1];

		idx = idx % stackStrideOutput[i];
	}

	uint stackOffsetA = 0;
	uint stackOffsetB = 0;
	for (int i = 0; i < NUM_STACK_DIMENSIONS; i++)
	{
		stackOffsetA += stackCoordA[i] * StackShapeA_StackShapeB_StackStrideA_StackStrideB[i][2];
		stackOffsetB += stackCoordB[i] * StackShapeA_StackShapeB_StackStrideA_StackStrideB[i][3];
	}

	return uint3(
		stackOffsetA	* MxK,
		stackOffsetB	* KxN,
		GroupID.z		* MxN);
#endif
}

WORK_TYPE GetBetaTimesC(const uint3 DispatchThreadID)
{
// Must correspond to EGemmCScalar defined in NNEHlslShadersGemmCS.h
#if C_SCALAR == 0
	return Beta * READ(C[(DispatchThreadID.y % CHeight) * N + (DispatchThreadID.x % CWidth)]);
#elif C_SCALAR == 1
	return Beta * CScalar;
#else
	return 0;
#endif
}

// Must correspond to EGemmAlgorithm defined in NNEHlslShadersGemmCS.h
#if ALGORITHM < 4

#if ALGORITHM == 0
[numthreads(8, 8, 1)]
#elif ALGORITHM == 1
[numthreads(16, 16, 1)]
#elif ALGORITHM == 2
[numthreads(32, 32, 1)]
#elif ALGORITHM == 3
[numthreads(256, 1, 1)]
#endif
void Gemm(in const uint3 DispatchThreadID : SV_DispatchThreadID, in const uint3 GroupID : SV_GroupID)
{
	if (DispatchThreadID.y >= M || DispatchThreadID.x >= N)
	{
		return;
	}
	
	uint AIdx;
	uint AIdxIncrement;
	if (TransA == 0)
	{
		AIdx = DispatchThreadID.y * K; // Index of the first element in the row
		AIdxIncrement = 1; // The loop goes over the whole row
	}
	else
	{
		AIdx = DispatchThreadID.y; // Index of the first element in the column
		AIdxIncrement = M; // The loop goes over the column, thus skip a full row
	}
	
	uint BIdx;
	uint BIdxIncrement;
	if (TransB == 0)
	{
		BIdx = DispatchThreadID.x; // Index of the first element in the column
		BIdxIncrement = N; // The loop goes over the column, thus skip a full row
	}
	else
	{
		BIdx = DispatchThreadID.x * K; // Index of the first element in the row
		BIdxIncrement = 1; // The loop goes over the whole row
	}

	const uint3 StackOffsets = GetMatrixStackOffsets(GroupID);
	AIdx += StackOffsets.x;
	BIdx += StackOffsets.y;
	
	WORK_TYPE Result = 0.0;
	for (int k = 0; k < K; k++)
	{
		Result += READ(A[AIdx]) * READ(B[BIdx]);
		AIdx += AIdxIncrement;
		BIdx += BIdxIncrement;
	}
	
	Y[StackOffsets.z + DispatchThreadID.y * N + DispatchThreadID.x] = WRITE(Alpha * Result + GetBetaTimesC(DispatchThreadID));
}

#elif ALGORITHM < 7

#if ALGORITHM == 4
#define GROUP_SIZE 8
#elif ALGORITHM == 5
#define GROUP_SIZE 16
#elif ALGORITHM == 6
#define GROUP_SIZE 32
#endif

groupshared WORK_TYPE SharedMemoryA[GROUP_SIZE * GROUP_SIZE];
groupshared WORK_TYPE SharedMemoryB[GROUP_SIZE * GROUP_SIZE];

[numthreads(GROUP_SIZE, GROUP_SIZE, 1)]
void Gemm(in const uint3 DispatchThreadID : SV_DispatchThreadID, in const uint3 GroupThreadID : SV_GroupThreadID, in const uint3 GroupID : SV_GroupID)
{
	// Calculate the number of steps each group needs to make to cover the full length of the matrix
	uint NumGroupSteps = (uint)ceil((float)K / (float)GROUP_SIZE);
	
	// Compute some variables required at multiple places
	uint SharedMemoryLineStartIdx = GroupThreadID.y * GROUP_SIZE;
	uint SharedMemoryIdx = SharedMemoryLineStartIdx + GroupThreadID.x;
	
	// Compute the lookup start indices and the increment which needs to be applied for each sub block
	uint AIdx;
	uint AIdxIncrement;
	if (TransA == 0)
	{
		AIdx = DispatchThreadID.y * K + GroupThreadID.x;
		AIdxIncrement = GROUP_SIZE;
	}
	else
	{
		AIdx = GroupThreadID.x * M + DispatchThreadID.y;
		AIdxIncrement = GROUP_SIZE * M;
	}
	
	uint BIdx;
	uint BIdxIncrement;
	if (TransB == 0)
	{
		BIdx = GroupThreadID.y * N + DispatchThreadID.x;
		BIdxIncrement = GROUP_SIZE * N;
	}
	else
	{
		BIdx = DispatchThreadID.x * K + GroupThreadID.y;
		BIdxIncrement = GROUP_SIZE;
	}

	const uint3 StackOffsets = GetMatrixStackOffsets(GroupID);
	AIdx += StackOffsets.x;
	BIdx += StackOffsets.y;
	
	// Multiply and accumulate the blocks along one dimension of the matrices
	WORK_TYPE Result = 0.0;
	uint GroupIdx = 0;
	for (int GroupStep = 0; GroupStep < NumGroupSteps; GroupStep++) 
	{
		// Only sync and update inidces starting with the second iteration
		if(GroupStep > 0)
		{
			// Increment the indices
			GroupIdx += GROUP_SIZE;
			AIdx += AIdxIncrement;
			BIdx += BIdxIncrement;
			
			// Make sure computation is done in the whole group before overwriting shared memory
			GroupMemoryBarrierWithGroupSync();
		}
		
		// Load the shared memory after applying the transpose and set overflow elements to zero
		WORK_TYPE Temp = 0;
		if (DispatchThreadID.y < M && GroupIdx + GroupThreadID.x < K)
		{
			Temp = READ(A[AIdx]);
		}
		SharedMemoryA[SharedMemoryIdx] = Temp;

		Temp = 0;
		if (DispatchThreadID.x < N && GroupIdx + GroupThreadID.y < K)
		{
			Temp = READ(B[BIdx]);
		}
		SharedMemoryB[SharedMemoryIdx] = Temp;
		
		// Make sure all memory has been loaded before running the multiplication
		GroupMemoryBarrierWithGroupSync();
		
		// Multiply the groups and accumulate the result
		// This works also at matrix borders, as overflow elements are set to zero
		uint SharedMemoryBIdx = GroupThreadID.x;
		for (int kGroup = 0; kGroup < GROUP_SIZE; kGroup++)
		{
			Result += SharedMemoryA[SharedMemoryLineStartIdx + kGroup] * SharedMemoryB[SharedMemoryBIdx];
			SharedMemoryBIdx += GROUP_SIZE;
		}
	}
	
	// Only now return out of bound threads as they were needed to load shared memory above
	if (DispatchThreadID.y >= M || DispatchThreadID.x >= N)
	{
		return;
	}
	
	Y[StackOffsets.z + DispatchThreadID.y * N + DispatchThreadID.x] = WRITE(Alpha * Result + GetBetaTimesC(DispatchThreadID));
}

#else

#if ALGORITHM == 7
#define GROUP_SIZE_X 		16
#define GROUP_SIZE_Y 		1
#define LOAD_PER_THREAD 	16
#elif ALGORITHM == 8
#define GROUP_SIZE_X 		16
#define GROUP_SIZE_Y 		2
#define LOAD_PER_THREAD 	8
#elif ALGORITHM == 9
#define GROUP_SIZE_X 		32
#define GROUP_SIZE_Y 		1
#define LOAD_PER_THREAD 	32
#elif ALGORITHM == 10
#define GROUP_SIZE_X 		32
#define GROUP_SIZE_Y 		2
#define LOAD_PER_THREAD 	16
#elif ALGORITHM == 11
#define GROUP_SIZE_X 		32
#define GROUP_SIZE_Y 		4
#define LOAD_PER_THREAD 	8
#elif ALGORITHM == 12
#define GROUP_SIZE_X 		64
#define GROUP_SIZE_Y 		2
#define LOAD_PER_THREAD 	32
#elif ALGORITHM == 13
#define GROUP_SIZE_X 		64
#define GROUP_SIZE_Y 		4
#define LOAD_PER_THREAD 	16
#elif ALGORITHM == 14
#define GROUP_SIZE_X 		64
#define GROUP_SIZE_Y 		8
#define LOAD_PER_THREAD 	8
#endif

// Use the larger of group sizes to define the shared memory size
// Threads will do multiple loads over the smaller dimension to fill the memory completely
groupshared WORK_TYPE SharedMemoryA[GROUP_SIZE_X * GROUP_SIZE_X];
groupshared WORK_TYPE SharedMemoryB[GROUP_SIZE_X * GROUP_SIZE_X];

[numthreads(GROUP_SIZE_X, GROUP_SIZE_Y, 1)]
void Gemm(in const uint3 DispatchThreadID : SV_DispatchThreadID, in const uint3 GroupThreadID : SV_GroupThreadID, in const uint3 GroupID : SV_GroupID)
{
	// Initialize the result
	WORK_TYPE Results[LOAD_PER_THREAD];
	for (int LoadIdx = 0; LoadIdx < LOAD_PER_THREAD; LoadIdx++)
	{
        Results[LoadIdx] = 0.0f;
    }
	
	// Calculate the number of steps each group needs to make to cover the full length of the matrix
	uint NumGroupSteps = (uint)ceil((float)K / (float)GROUP_SIZE_X);
	
	// Compute some variables required at multiple places
	uint SharedMemoryTransposedLineIdx = GroupThreadID.x * GROUP_SIZE_X;
	uint SharedMemoryStartIdx = GroupThreadID.y * GROUP_SIZE_X + GroupThreadID.x;
	uint SharedIdxIncrement = GROUP_SIZE_Y * GROUP_SIZE_X;
	uint GroupStartRow = GroupID.y * GROUP_SIZE_X;
	uint GroupLoadOffset = GroupThreadID.y * LOAD_PER_THREAD;
	
	// Compute the lookup start indices and the increment which needs to be applied for each sub block
	uint AIdx;
	uint AIdxOuterIncrement; // Increment added in the group loop
	uint AIdxInnerIncrement; // Increment added in the load loop
	if (TransA == 0)
	{
		AIdx = (GroupStartRow + GroupThreadID.y) * K + GroupThreadID.x;
		AIdxOuterIncrement = GROUP_SIZE_X;
		AIdxInnerIncrement = GROUP_SIZE_Y * K;
	}
	else
	{
		AIdx = GroupThreadID.x * M + (GroupStartRow + GroupThreadID.y);
		AIdxOuterIncrement = GROUP_SIZE_X * M;
		AIdxInnerIncrement = GROUP_SIZE_Y;
	}
	
	uint BIdx;
	uint BIdxOuterIncrement; // Increment added in the group loop
	uint BIdxInnerIncrement; // Increment added in the load loop
	if (TransB == 0)
	{
		BIdx = GroupThreadID.y * N + DispatchThreadID.x;
		BIdxOuterIncrement = GROUP_SIZE_X * N;
		BIdxInnerIncrement = GROUP_SIZE_Y * N;
	}
	else
	{
		BIdx = DispatchThreadID.x * K + GroupThreadID.y;
		BIdxOuterIncrement = GROUP_SIZE_X;
		BIdxInnerIncrement = GROUP_SIZE_Y;
	}

	const uint3 StackOffsets = GetMatrixStackOffsets(GroupID);
	AIdx += StackOffsets.x;
	BIdx += StackOffsets.y;

	// Multiply and accumulate the blocks
	uint GroupIdx = 0;
	for (uint GroupStep = 0; GroupStep < NumGroupSteps; GroupStep++) 
	{
		// Prepare some shared variables needed inside this loop
		uint SharedIdx = SharedMemoryStartIdx;
		uint AIdxOffset = 0;
		uint BIdxOffset = 0;
		
		uint GroupRow = GroupThreadID.y;
		uint ThreadIDY = GroupStartRow + GroupRow;
		
		// Only sync and update inidces starting with the second iteration
		if (GroupStep > 0)
		{
			// increment indices
			GroupIdx += GROUP_SIZE_X;
			AIdx += AIdxOuterIncrement;
			BIdx += BIdxOuterIncrement;
			
			// Make sure computation is done in the whole group before overwriting shared memory
			GroupMemoryBarrierWithGroupSync();
		}
		
		for (int LoadIdx = 0; LoadIdx < LOAD_PER_THREAD; LoadIdx++)
		{
			// Load the shared memory after applying the transpose and set overflow elements to zero
			WORK_TYPE Temp = 0;
			if (ThreadIDY < M && GroupIdx + GroupThreadID.x < K)
			{
				Temp = READ(A[AIdx + AIdxOffset]);
			}
			SharedMemoryA[SharedIdx] = Temp;
			
			Temp = 0;
			if (GroupIdx + GroupRow < K && DispatchThreadID.x < N)
			{
				Temp = READ(B[BIdx + BIdxOffset]);
			}
			SharedMemoryB[SharedIdx] = Temp;
			
			// Increment the thread y coordinate virtually by one group size
			GroupRow += GROUP_SIZE_Y;
			ThreadIDY += GROUP_SIZE_Y;
			
			// Increment the shared index pointer to advance to the next subblock
			SharedIdx += SharedIdxIncrement;
			AIdxOffset += AIdxInnerIncrement;
			BIdxOffset += BIdxInnerIncrement;
		}
		
		// Make sure all memory has been loaded before running the multiplication
		GroupMemoryBarrierWithGroupSync();
		
		// multiply and accumulate
		uint StartB = GroupLoadOffset;
		for (int kGroup = 0; kGroup < GROUP_SIZE_X; kGroup++)
		{
			WORK_TYPE AVal = SharedMemoryA[SharedMemoryTransposedLineIdx + kGroup];
			for (int LoadIdx = 0; LoadIdx < LOAD_PER_THREAD; LoadIdx++)
			{
				Results[LoadIdx] += AVal * SharedMemoryB[StartB + LoadIdx];
			}
			StartB += GROUP_SIZE_X;
		}
	}
	
	// Only now return out of bound threads as they were needed to load shared memory above
	uint Row = GroupStartRow + GroupThreadID.x;
	if (Row >= M)
	{
		return;
	}
	
	// Save back the results
	uint RowIdx = Row * N;
	uint ColumnStart = GroupID.x * GROUP_SIZE_X + GroupLoadOffset;
	for (int LoadIdx = 0; LoadIdx < LOAD_PER_THREAD; LoadIdx++)
	{
		uint Column = ColumnStart + LoadIdx;
		
		// Return as soon as the column is out of bound as further loops will be outside too
		if (Column >= N)
		{
			return;
		}
		
		Y[StackOffsets.z + RowIdx + Column] = WRITE(Alpha * Results[LoadIdx] + GetBetaTimesC(uint3(Column, Row, 0)));
    }
}

#endif
