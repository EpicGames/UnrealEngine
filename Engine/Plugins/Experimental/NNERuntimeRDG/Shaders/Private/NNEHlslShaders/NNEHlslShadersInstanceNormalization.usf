// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "/Engine/Public/Platform.ush"
 
#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x
 
Buffer<WORK_TYPE> Input;
Buffer<WORK_TYPE> InputScale;
Buffer<WORK_TYPE> InputBias;
Buffer<WORK_TYPE> InputMean;
Buffer<WORK_TYPE> InputInvStdDev;
RWBuffer<WORK_TYPE> Output;
uint Num;
uint ThreadCountX;
uint InstanceSize;
uint ChannelSize;
 
[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void InstanceNormalization(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;

	if (Index < Num)
	{
		const uint InstanceIdx = Index/InstanceSize;
		const uint ChannelIdx = InstanceIdx%ChannelSize;
		WORK_TYPE Mean = READ(InputMean[InstanceIdx]);
		WORK_TYPE InvStdDev = READ(InputInvStdDev[InstanceIdx]);
		WORK_TYPE Scale = READ(InputScale[ChannelIdx]);
		WORK_TYPE Bias = READ(InputBias[ChannelIdx]);
		Output[Index] = WRITE((READ(Input[Index]) - Mean) * Scale * InvStdDev + Bias);
	}
}