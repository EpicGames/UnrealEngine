// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

Buffer<BUFFER_TYPE> Input;
uint4 InputTensorInfo[NUM_DIMENSIONS];
Buffer<BUFFER_TYPE> InputScale;
uint4 ScaleTensorInfo[NUM_DIMENSIONS];
Buffer<BUFFER_TYPE> InputBias;
uint4 BiasTensorInfo[NUM_DIMENSIONS];
Buffer<WORK_TYPE> InputMean;
Buffer<WORK_TYPE> InputInvStdDev;
RWBuffer<BUFFER_TYPE> Output;

float Epsilon;
int Axis;
uint ThreadCountX;
uint LayerSize;
uint Num;

#define STRIDE_IDX 0

#define STATIC_LOOP(Var, From, To) \
[unroll] \
for(uint Var = From; Var < To; ++Var)

void GetDimIterator(const uint GlobalIdx, const uint4 TensorInfo[NUM_DIMENSIONS], out uint DimIterator[NUM_DIMENSIONS])
{
	uint Offset = GlobalIdx;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		uint Remainder;
		DivMod(Offset, TensorInfo[DimIdx][STRIDE_IDX], DimIterator[DimIdx], Remainder);
		Offset = Remainder;
	}
}

uint GetInnerIdx(const uint DimIterator[NUM_DIMENSIONS], const uint4 TensorInfo[NUM_DIMENSIONS], const uint Axis)
{
	uint Offset = 0;
	for(uint DimIdx = 0; DimIdx < NUM_DIMENSIONS - Axis; ++DimIdx)
	{
		Offset += DimIterator[DimIdx + Axis] * TensorInfo[DimIdx][STRIDE_IDX];
	}
	return Offset;
}


[numthreads(THREADGROUP_SIZE, 1, 1)]
void LayerNormalization(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint ElementIdx = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	if(ElementIdx < Num)
	{
		const uint LayerIdx = ElementIdx / LayerSize;
		
		WORK_TYPE Mean = InputMean[LayerIdx];
		WORK_TYPE InvStdDev = InputInvStdDev[LayerIdx];

		BUFFER_TYPE Normalized = (BUFFER_TYPE) ( (READ(Input[ElementIdx]) - Mean) * InvStdDev );

		uint InputDimIterator[NUM_DIMENSIONS];
		GetDimIterator(ElementIdx, InputTensorInfo, InputDimIterator);

		BUFFER_TYPE NormalizedScaled = Normalized * READ( InputScale[GetInnerIdx(InputDimIterator, ScaleTensorInfo, Axis)]);
		#if HAS_B == 0
			Output[ElementIdx] = WRITE( NormalizedScaled );
		#else
			Output[ElementIdx] = WRITE( NormalizedScaled + READ( InputBias[GetInnerIdx(InputDimIterator, BiasTensorInfo, Axis)] ) );
		#endif
	}
}
