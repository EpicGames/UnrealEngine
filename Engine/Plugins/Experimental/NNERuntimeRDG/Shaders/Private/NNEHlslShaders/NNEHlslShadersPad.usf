// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSIONS];
float Value;
uint Num;
uint ThreadCountX;

#define INPUT_STRIDE 0
#define OUTPUT_STRIDE 1
#define INPUT_SIZE 2
#define PRE_PAD 3

// Must correspond to EPadMode defined in NNEHlslShadersPadCS.h
#define CONSTANT_MODE 0
#define REFLECT_MODE 1
#define EDGE_MODE 2

#define STATIC_LOOP(Var, From, To) \
[unroll] \
for(uint Var = From; Var < To; ++Var)

void GetOutputDimIterator(const uint GlobalIdx, out uint DimIterator[NUM_DIMENSIONS])
{
	uint Offset = GlobalIdx;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		uint Remainder;
		DivMod(Offset, TensorInfo[DimIdx][OUTPUT_STRIDE], DimIterator[DimIdx], Remainder);
		Offset = Remainder;
	}
}

uint GetInputGlobalIdx(const uint DimIterator[NUM_DIMENSIONS])
{
	uint Offset = 0;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		Offset += DimIterator[DimIdx] * TensorInfo[DimIdx][INPUT_STRIDE];
	}
	return Offset;
}

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Pad(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint OutputDimIterator[NUM_DIMENSIONS];
		GetOutputDimIterator(Index, OutputDimIterator);

		uint InputDimIterator[NUM_DIMENSIONS];

		for(uint DimIdx = 0; DimIdx < NUM_DIMENSIONS; ++DimIdx)
		{
			const int InputDimSize = (int) TensorInfo[DimIdx][INPUT_SIZE];
			
			int PrePad = asint(TensorInfo[DimIdx][PRE_PAD]); // Decode PrePad to int (can be negative)
			int CurDimIterator = (int) OutputDimIterator[DimIdx] - PrePad;
			
			if (CurDimIterator < 0 || CurDimIterator >= InputDimSize)
			{
				#if MODE == CONSTANT_MODE
					Output[Index] = Value;
					return;
				#elif MODE == REFLECT_MODE
					if (CurDimIterator >= InputDimSize)
					{
						const int OverCnt = CurDimIterator - (InputDimSize - 2);
						CurDimIterator = InputDimSize - OverCnt;
					}
					else if (CurDimIterator < 0)
					{
						CurDimIterator = -CurDimIterator;
					}
				#endif
				//MODE == EDGE_MODE + clamping large reflect indices
				CurDimIterator = min(CurDimIterator, InputDimSize - 1);
				CurDimIterator = max(CurDimIterator, 0);
			}

			InputDimIterator[DimIdx] = (uint) CurDimIterator;
		}

		Output[Index] = Input[GetInputGlobalIdx(InputDimIterator)];
	}
}
