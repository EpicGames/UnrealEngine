// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

#define NUM_DIMENSION (NUM_SPATIAL_DIMENSIONS+2)

Buffer<float> Input;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSION];
uint4 SpatialInfo[NUM_SPATIAL_DIMENSIONS];
uint Num;
uint ThreadCountX;
uint KernelVolume;

#define OUTPUT_STRIDE 0
#define INPUT_STRIDE 1
#define INPUT_SIZE 2

#define SPATIAL_STRIDE 0
#define SPATIAL_KERNEL 1
#define SPATIAL_PAD_START 2
#define SPATIAL_DILATION 3

#define MAX_POOL 0
#define AVERAGE_POOL 1

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Pool(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	const uint SpatialDimOffset = 2;
	
	if (Index < Num)
	{
		//Compute indices on output tensors
		uint Offset = Index;
		uint OutputIndices[NUM_DIMENSION];
		for (uint dim = 0; dim < NUM_DIMENSION; ++dim)
		{
			uint OutDimIdx, R;
			DivMod(Offset, TensorInfo[dim][OUTPUT_STRIDE], OutputIndices[dim], R);
			Offset = R;
		}

		//Compute spatial region to apply pooling on input
		uint SpatialDimStarts[NUM_SPATIAL_DIMENSIONS];
		uint SpatialDimEnds[NUM_SPATIAL_DIMENSIONS];
		for (uint spatialDim = 0; spatialDim < NUM_SPATIAL_DIMENSIONS; ++spatialDim)
		{
			const uint Dim = SpatialDimOffset + spatialDim;
			const int SpatialDimStart = ((int)OutputIndices[Dim] * (int)SpatialInfo[spatialDim][SPATIAL_STRIDE]) - (int)SpatialInfo[spatialDim][SPATIAL_PAD_START];
			const int SpatialDimEnd = SpatialDimStart + (int)SpatialInfo[spatialDim][SPATIAL_KERNEL] * (int)SpatialInfo[spatialDim][SPATIAL_DILATION];

			SpatialDimStarts[spatialDim] = (uint)max(SpatialDimStart, (int)0);
			SpatialDimEnds[spatialDim] = (uint)min(SpatialDimEnd, (int)TensorInfo[Dim][INPUT_SIZE]);
		}

		const uint BaseOffset = (TensorInfo[0][INPUT_STRIDE] * OutputIndices[0]) + (TensorInfo[1][INPUT_STRIDE] * OutputIndices[1]);
		uint SpatialDimIterator[NUM_SPATIAL_DIMENSIONS];

		//Apply pooling, looping other the input for all spatial dimensions of the input kernel
		float PooledValue = 0;
		int Count = 0;
		
		for (SpatialDimIterator[0] = SpatialDimStarts[0]; SpatialDimIterator[0] < SpatialDimEnds[0]; SpatialDimIterator[0] += SpatialInfo[0][SPATIAL_DILATION])
		{
		#if (NUM_SPATIAL_DIMENSIONS > 1)
		for (SpatialDimIterator[1] = SpatialDimStarts[1]; SpatialDimIterator[1] < SpatialDimEnds[1]; SpatialDimIterator[1] += SpatialInfo[1][SPATIAL_DILATION])
		{
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 2)
		for (SpatialDimIterator[2] = SpatialDimStarts[2]; SpatialDimIterator[2] < SpatialDimEnds[2]; SpatialDimIterator[2] += SpatialInfo[2][SPATIAL_DILATION])
		{
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 3)
		for (SpatialDimIterator[3] = SpatialDimStarts[3]; SpatialDimIterator[3] < SpatialDimEnds[3]; SpatialDimIterator[3] += SpatialInfo[3][SPATIAL_DILATION])
		{
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 4)
		for (SpatialDimIterator[4] = SpatialDimStarts[4]; SpatialDimIterator[4] < SpatialDimEnds[4]; SpatialDimIterator[4] += SpatialInfo[4][SPATIAL_DILATION])
		{
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 5)
		for (SpatialDimIterator[5] = SpatialDimStarts[5]; SpatialDimIterator[5] < SpatialDimEnds[5]; SpatialDimIterator[5] += SpatialInfo[5][SPATIAL_DILATION])
		{
		#endif

			uint InputIndex = BaseOffset;
			[unroll]
			for (uint spatialDim = 0; spatialDim < NUM_SPATIAL_DIMENSIONS; ++spatialDim)
			{
				InputIndex += TensorInfo[spatialDim+2][INPUT_STRIDE] * SpatialDimIterator[spatialDim];
			}
			float CurrentValue = Input[InputIndex];
			#if (POOL_OPERATOR_TYPE == MAX_POOL)
				//float MaxValue = max(PooledValue, CurrentValue); //Is not working properly on DX12 SM6 thus the ternary op for MaxValue
				float MaxValue = (PooledValue > CurrentValue) ? PooledValue : CurrentValue;
				PooledValue = (Count == 0) ? CurrentValue : MaxValue;
			#else //POOL_OPERATOR_TYPE == AVERAGE_POOL
				PooledValue += CurrentValue;
			#endif 
			++Count;

		#if (NUM_SPATIAL_DIMENSIONS > 5)
		}
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 4)
		}
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 3)
		}
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 2)
		}
		#endif
		#if (NUM_SPATIAL_DIMENSIONS > 1)
		}
		#endif
		}

		#if (POOL_OPERATOR_TYPE == AVERAGE_POOL)
			PooledValue /= (KernelVolume == 0) ? (float)Count : (float)KernelVolume;
		#endif

		Output[Index] = PooledValue;
	}
}
