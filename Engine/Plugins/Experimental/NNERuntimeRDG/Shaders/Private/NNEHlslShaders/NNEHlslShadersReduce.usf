// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

Buffer<WORK_TYPE> Input;
RWBuffer<WORK_TYPE> Output;
RWBuffer<BUFFER_TYPE> Output2;
int NumElemBeforeAxis;
int AxisSize;
int NumElemAfterAxis;
uint4 DispatchIdxAndStride;
float Epsilon;

// Must correspond to EReduceOperatorType defined in NNEHlslShadersReduceCS.h
#define TYPE_AVERAGE 0
#define TYPE_L1 1
#define TYPE_L2 2
#define TYPE_LOGSUMEXP 3
#define TYPE_MAX 4
#define TYPE_MIN 5
#define TYPE_PROD 6
#define TYPE_SUM 7
#define TYPE_SUMEXP 8
#define TYPE_AVERAGEINVSTDDEV 9

#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
	groupshared WORK_TYPE SharedMemoryMean[THREADGROUP_SIZE];
	groupshared WORK_TYPE SharedMemoryMean2[THREADGROUP_SIZE];
	groupshared WORK_TYPE SharedMemoryCount[THREADGROUP_SIZE];
#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTD
	groupshared WORK_TYPE SharedMemory[THREADGROUP_SIZE];

	WORK_TYPE InitValue(WORK_TYPE FirstValue)
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGE
			return 0;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L1
			return 0;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L2
			return 0;
		#elif REDUCE_OPERATOR_TYPE == TYPE_LOGSUMEXP
			return 0;
		#elif REDUCE_OPERATOR_TYPE == TYPE_MAX
			return FirstValue;
		#elif REDUCE_OPERATOR_TYPE == TYPE_MIN
			return FirstValue;
		#elif REDUCE_OPERATOR_TYPE == TYPE_PROD
			return 1;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUM
			return 0;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUMEXP
			return 0;
		#endif
	}

	WORK_TYPE ElementOp(WORK_TYPE Value)
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGE
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L1
			return abs(Value);
		#elif REDUCE_OPERATOR_TYPE == TYPE_L2
			return Value*Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_LOGSUMEXP
			return exp(Value);
		#elif REDUCE_OPERATOR_TYPE == TYPE_MAX
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_MIN
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_PROD
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUM
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUMEXP
			return exp(Value);
		#endif
	}

	WORK_TYPE CombineOp(WORK_TYPE ValueA, WORK_TYPE ValueB)
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGE
			return ValueA+ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L1
			return ValueA+ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L2
			return ValueA+ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_LOGSUMEXP
			return ValueA+ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_MAX
			return max(ValueA, ValueB);
		#elif REDUCE_OPERATOR_TYPE == TYPE_MIN
			return min(ValueA, ValueB);
		#elif REDUCE_OPERATOR_TYPE == TYPE_PROD
			return ValueA*ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUM
			return ValueA+ValueB;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUMEXP
			return ValueA+ValueB;
		#endif
	}

	WORK_TYPE FinalOp(WORK_TYPE Value)
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGE
			return Value/AxisSize;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L1
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_L2
			return sqrt(Value);
		#elif REDUCE_OPERATOR_TYPE == TYPE_LOGSUMEXP
			return log(Value);
		#elif REDUCE_OPERATOR_TYPE == TYPE_MAX
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_MIN
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_PROD
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUM
			return Value;
		#elif REDUCE_OPERATOR_TYPE == TYPE_SUMEXP
			return Value;
		#endif
	}
#endif //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTD

[numthreads(THREADGROUP_SIZE, 1, 1)]
void Reduce(in const uint3 DispatchThreadID : SV_DispatchThreadID, in const uint3 GroupThreadID : SV_GroupThreadID)
{
	const uint IdxAfterAxis = DispatchIdxAndStride.x * DispatchIdxAndStride.z +  DispatchThreadID.y;
	const uint IdxBeforeAxis =  DispatchIdxAndStride.y * DispatchIdxAndStride.w +  DispatchThreadID.z;
	const int InputStartIdx = (IdxBeforeAxis * AxisSize * NumElemAfterAxis) + IdxAfterAxis;
	const int OutputIdx     = (IdxBeforeAxis * 1        * NumElemAfterAxis) + IdxAfterAxis;

	// Reduction using sharedmem and THREADGROUP_SIZE threads along the reduced dimension
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
			WORK_TYPE Mean = 0;
			WORK_TYPE Mean2 = 0;
			int Count = 0;
		#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
			WORK_TYPE FirstValue = READ(Input[InputStartIdx]);
			WORK_TYPE Reduced = InitValue(FirstValue);
		#endif


			for (int AxisIdx = GroupThreadID.x; AxisIdx < AxisSize; AxisIdx += THREADGROUP_SIZE)
			{
				WORK_TYPE Value = READ(Input[(AxisIdx * NumElemAfterAxis) + InputStartIdx]);
			
				#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
					// Calculate mean and variance using Welford's online algorithm
					// https://en.wikipedia.org/wiki/Algorithms_for_calculating_variance
					Count += 1;
					WORK_TYPE Delta = Value - Mean;
					Mean += Delta / (WORK_TYPE)Count;
					WORK_TYPE Delta2 = Value - Mean;
					Mean2 += Delta * Delta2;
				#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
					Value = ElementOp(Value);
					Reduced = CombineOp(Reduced, Value);
				#endif
			}

		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
			SharedMemoryMean[GroupThreadID.x] = Mean;
			SharedMemoryMean2[GroupThreadID.x] = Mean2;
			SharedMemoryCount[GroupThreadID.x] = Count;
		#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
			SharedMemory[GroupThreadID.x] = Reduced;
		#endif
	}

	// Wait until all threads wrote result to shared memory
	GroupMemoryBarrierWithGroupSync();

	// Use one thread to calculate final result
	if (GroupThreadID.x == 0)
	{
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
			WORK_TYPE Mean = 0;
			WORK_TYPE Mean2 = 0;
			int Count = 0;
		#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
			WORK_TYPE FirstCurrReduced = SharedMemory[0];
			WORK_TYPE Reduced = InitValue(FirstCurrReduced);
		#endif

			UNROLL
			for (int Idx = 0; Idx < THREADGROUP_SIZE; Idx++)
			{
				#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
					WORK_TYPE CurrMean = SharedMemoryMean[Idx];
					WORK_TYPE CurrMean2 = SharedMemoryMean2[Idx];
					int CurrCount = SharedMemoryCount[Idx];
					int NewCount = Count + CurrCount;
					WORK_TYPE Delta = CurrMean - Mean;
					Mean = (CurrCount * CurrMean + Count * Mean) / NewCount;
					Mean2 += CurrMean2 + Delta * Delta * (WORK_TYPE)Count * (WORK_TYPE)CurrCount / (WORK_TYPE)NewCount;
					Count = NewCount;
				#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
					WORK_TYPE CurrReduced = SharedMemory[Idx];
					Reduced = CombineOp(Reduced, CurrReduced);
				#endif
			}

		// Write reduced result
		#if REDUCE_OPERATOR_TYPE == TYPE_AVERAGEINVSTDDEV
			WORK_TYPE Variance = Mean2 / (WORK_TYPE)Count;
			WORK_TYPE InvStdev = (WORK_TYPE)1 / sqrt(Variance + Epsilon);
			Output[OutputIdx] = Mean;
			Output2[OutputIdx] = InvStdev;
		#else //REDUCE_OPERATOR_TYPE != TYPE_AVERAGEINVSTDDEV
			Reduced = FinalOp(Reduced);
			Output[OutputIdx] = Reduced;
		#endif
	}
}