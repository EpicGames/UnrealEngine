// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"
#include "/Engine/Private/DoubleFloat.ush"

static const uint NumPointsLinear = pow(2,NUM_DIMENSIONS);

Buffer<float> Input;
uint4 InputTensorInfo[NUM_DIMENSIONS];
RWBuffer<float> Output;
uint4 OutputTensorInfo[NUM_DIMENSIONS];
float4 ScalesData[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;

#define STRIDE_IDX 0
#define SIZE_IDX 1
#define ADJUSTMENT_IDX 2
#define SCALES_IDX 0
#define ROI_START_IDX 2
#define ROI_END_IDX 3

// Corresponds to ECoordTransMode
#define COORD_TRANS_HALF_PIXEL 0
#define COORD_TRANS_HALF_PIXEL_SYMMETRIC 1
#define COORD_TRANS_PYTORCH_HALF_PIXEL 2
#define COORD_TRANS_ALIGN_CORNERS 3
#define COORD_TRANS_ASYMMETRIC 4
#define COORD_TRANS_TF_HALF_PIXEL_FOR_NN 5
#define COORD_TRANS_TF_CROP_AND_RESIZE 6

// Corresponds to ECoordTransMode enum
#define NEAREST_MODE_ROUND_PREFER_FLOOR 0
#define NEAREST_MODE_ROUND_PREFER_CEIL 1
#define NEAREST_MODE_FLOOR 2
#define NEAREST_MODE_CEIL 3

// Corresponds to EMode enum
#define MODE_NEAREST 0
#define MODE_LINEAR 1
#define MODE_CUBIC 2

#define STATIC_LOOP(Var, From, To) \
[unroll] \
for(uint Var = From; Var < To; ++Var)

void FromIdxToIterator(const uint Idx, const uint4 TensorInfo[NUM_DIMENSIONS], out uint Iterator[NUM_DIMENSIONS])
{
	uint Offset = Idx;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		uint Remainder;
		DivMod(Offset, TensorInfo[DimIdx][STRIDE_IDX], Iterator[DimIdx], Remainder);
		Offset = Remainder;
	}
}

uint FromIteratorToIdx(const uint4 TensorInfo[NUM_DIMENSIONS], const uint Iterator[NUM_DIMENSIONS])
{
	uint Offset = 0;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		Offset += Iterator[DimIdx] * TensorInfo[DimIdx][STRIDE_IDX];
	}
	return Offset;
}

float TransformCoord(float Resized, uint DimIdx)
{
	float Scale = ScalesData[DimIdx][SCALES_IDX];

	// Computations of formulas which combine sums and divisions are done in double precision since single precision is affected by some heavy optimizations
	// that lead to very inaccurate results. This way we obtain the same input pixels as the ORT implementations.

	#if COORD_TRANS_MODE == COORD_TRANS_HALF_PIXEL
		return DFSubtractDemote(DFDivide(DFTwoSum(Resized, 0.5f), Scale), 0.5f);
	#elif COORD_TRANS_MODE == COORD_TRANS_HALF_PIXEL_SYMMETRIC
		float Adjustment = asfloat(OutputTensorInfo[DimIdx][ADJUSTMENT_IDX]); // Decode adjustment to float
		FDFScalar Center = DFDivide(InputTensorInfo[DimIdx][SIZE_IDX], 2.0f);
		return DFSubtractDemote(DFAdd(DFMultiply(Center, DFSubtract(1.0f, Adjustment)), DFDivide(DFTwoSum(Resized, 0.5f), Scale)), 0.5f);
	#elif COORD_TRANS_MODE == COORD_TRANS_PYTORCH_HALF_PIXEL
		return OutputTensorInfo[DimIdx][SIZE_IDX] > 1 ? DFSubtractDemote(DFDivide(DFTwoSum(Resized, 0.5f), Scale), 0.5f) : 0.0f;
	#elif COORD_TRANS_MODE == COORD_TRANS_ALIGN_CORNERS
		return DFMultiplyDemote(Resized, DFDivide((InputTensorInfo[DimIdx][SIZE_IDX] - 1), (OutputTensorInfo[DimIdx][SIZE_IDX] - 1)));
	#elif COORD_TRANS_MODE == COORD_TRANS_ASYMMETRIC
		return DFDemote(DFDivide(Resized, Scale));
	#elif COORD_TRANS_MODE == COORD_TRANS_TF_HALF_PIXEL_FOR_NN
		return DFDemote(DFDivide(DFTwoSum(Resized, 0.5f), Scale));
	#else // COORD_TRANS_MODE == COORD_TRANS_TF_CROP_AND_RESIZE
		float Start = asfloat(InputTensorInfo[DimIdx][ROI_START_IDX]); // Decode ROI start to float
		float End = asfloat(InputTensorInfo[DimIdx][ROI_END_IDX]); // Decode ROI end to float
		return OutputTensorInfo[DimIdx][SIZE_IDX] > 1 ? 
			DFAddDemote(DFMultiply(Start, (InputTensorInfo[DimIdx][SIZE_IDX] - 1)), DFDivide(DFMultiply(DFMultiply(Resized, DFSubtract(End, Start)), (InputTensorInfo[DimIdx][SIZE_IDX] - 1)), (OutputTensorInfo[DimIdx][SIZE_IDX] - 1))) 
			: 
			DFMultiplyDemote(DFMultiply(0.5f, DFTwoSum(Start, End)), (InputTensorInfo[DimIdx][SIZE_IDX] - 1));
	#endif

}

void GetNearestInput(uint OutputIndex, out uint Iterator[NUM_DIMENSIONS])
{
	uint OutputIterator[NUM_DIMENSIONS];
	
	FromIdxToIterator(OutputIndex, OutputTensorInfo, OutputIterator);

	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		float InputCoord = TransformCoord((float) OutputIterator[DimIdx], DimIdx);
		
		int Dim = (int)
			#if NEAREST_MODE == NEAREST_MODE_FLOOR
				floor(InputCoord)
			#elif NEAREST_MODE == NEAREST_MODE_CEIL
				ceil(InputCoord)
			#elif NEAREST_MODE == NEAREST_MODE_ROUND_PREFER_FLOOR
				( InputCoord == (int) InputCoord + 0.5f ? floor(InputCoord) : round(InputCoord) )
			#else // NEAREST_MODE == NEAREST_MODE_ROUND_PREFER_CEIL
				( InputCoord == (int) InputCoord + 0.5f ? ceil(InputCoord) : round(InputCoord) )
			#endif
			;
		
		Dim = clamp(Dim, 0, (int) InputTensorInfo[DimIdx][SIZE_IDX] - 1);
		
		Iterator[DimIdx] = (uint) Dim;
	}

}

#if MODE == MODE_NEAREST

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Resize(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint OutputIndex = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (OutputIndex < Num)
	{
		uint InputIterator[NUM_DIMENSIONS];

		GetNearestInput(OutputIndex, InputIterator);

		uint InputIndex = FromIteratorToIdx(InputTensorInfo, InputIterator);		

		Output[OutputIndex] = Input[InputIndex];

	}
}

#elif MODE == MODE_LINEAR

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Resize(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint OutputIndex = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (OutputIndex < Num)
	{
		float Weights[NUM_DIMENSIONS];
		uint InputIterator[NUM_DIMENSIONS];

		{
			uint OutputIterator[NUM_DIMENSIONS];
		
			FromIdxToIterator(OutputIndex, OutputTensorInfo, OutputIterator);

			STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
			{
				float InputCoord = InputTensorInfo[DimIdx][SIZE_IDX] == 1 ? 
						(float) OutputIterator[DimIdx]
						:
						TransformCoord((float) OutputIterator[DimIdx], DimIdx);
				InputCoord = max(0.0f, min(InputCoord, (float)(InputTensorInfo[DimIdx][SIZE_IDX] - 1)));
				InputIterator[DimIdx] = (uint) floor(InputCoord);

				Weights[DimIdx] = (InputIterator[DimIdx] >= InputTensorInfo[DimIdx][SIZE_IDX] - 1) ? 0.5f : frac(InputCoord);
			}

		}

		float OutputValue = 0.0f;

		STATIC_LOOP(PointIdx, 0, NumPointsLinear)
		{
			uint PointCoords[NUM_DIMENSIONS];
			float Factor = 1.0f;

			{

				STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
				{
					uint PointCoord = (PointIdx & (1u << DimIdx)) >> DimIdx;
					PointCoords[DimIdx] = InputIterator[DimIdx] + PointCoord;
					if(PointCoords[DimIdx] >= (InputTensorInfo[DimIdx][SIZE_IDX] - 1))
					{
						PointCoords[DimIdx] = InputTensorInfo[DimIdx][SIZE_IDX] - 1;
					}
					Factor *= PointCoord == 1 ? Weights[DimIdx] : (1.0f - Weights[DimIdx]);

				}

			}

			float PointValue = Input[FromIteratorToIdx(InputTensorInfo, PointCoords)];
			OutputValue += PointValue * Factor;
		}

		Output[OutputIndex] = OutputValue;
	}
}


#endif