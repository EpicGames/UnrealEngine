// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

Buffer<int> InputIndices;
Buffer<BUFFER_TYPE> InputUpdates;
RWBuffer<BUFFER_TYPE> Output;
uint4 DataTensorInfo[NUM_DIMENSIONS];

uint Num;
uint ThreadCountX;
uint PartialIndexRank; // k
uint SliceSize; // Obtained by multiplying elements in data.shape[k:r-1]

#define STRIDE_IDX 0
#define SIZE_IDX 1

// Corresponds to EScatterNDReductionType defined in NNEHlslShadersScatterNDCS.h
#define TYPE_NONE 0
#define TYPE_ADD 1
#define TYPE_MUL 2
#define TYPE_MAX 3
#define TYPE_MIN 4

uint NormalizeIdx(int Idx, uint DimSize)
{
	return Idx < 0 ? (uint) (Idx + DimSize) : (uint) Idx;
}

// int64_t is only available from SM 6.0, therefore here we need to manually cast to int32.
int CastInt64ToInt32(uint LSW, int MSW)
{
	return (MSW >> 31) == 1 ? - (int) LSW : (int) LSW;
}

#define STATIC_LOOP(Var, From, To) \
[unroll] \
for(uint Var = From; Var < To; ++Var)

void FromIdxToIterator(const uint Idx, const uint4 TensorInfo[NUM_DIMENSIONS], out uint Iterator[NUM_DIMENSIONS])
{
	uint Offset = Idx;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		uint Remainder;
		DivMod(Offset, TensorInfo[DimIdx][STRIDE_IDX], Iterator[DimIdx], Remainder);
		Offset = Remainder;
	}
}

uint FromIteratorToIdx(const uint4 TensorInfo[NUM_DIMENSIONS], const uint Iterator[NUM_DIMENSIONS])
{
	uint Offset = 0;
	STATIC_LOOP(DimIdx, 0, NUM_DIMENSIONS)
	{
		Offset += Iterator[DimIdx] * TensorInfo[DimIdx][STRIDE_IDX];
	}
	return Offset;
}

void ApplyReduction(const uint OutputIdx, const uint UpdatesIdx)
{
	#if REDUCE_OPERATOR_TYPE == TYPE_NONE
		Output[OutputIdx] = WRITE(READ(InputUpdates[UpdatesIdx]));
	#elif REDUCE_OPERATOR_TYPE == TYPE_ADD
		Output[OutputIdx] = WRITE(READ(Output[OutputIdx]) + READ(InputUpdates[UpdatesIdx]));
	#elif REDUCE_OPERATOR_TYPE == TYPE_MUL
		Output[OutputIdx] = WRITE(READ(Output[OutputIdx]) * READ(InputUpdates[UpdatesIdx]));
	#elif REDUCE_OPERATOR_TYPE == TYPE_MAX
		Output[OutputIdx] = WRITE(max(READ(Output[OutputIdx]), READ(InputUpdates[UpdatesIdx])));
	#elif REDUCE_OPERATOR_TYPE == TYPE_MIN
		Output[OutputIdx] = WRITE(min(READ(Output[OutputIdx]), READ(InputUpdates[UpdatesIdx])));
	#endif
}


[numthreads(THREADGROUP_SIZE, 1, 1)]
void ScatterND(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	// See https://github.com/onnx/onnx/blob/main/docs/Operators.md#ScatterND
	// Parallelizing over updates, of shape: indices.shape[0:q-2] ++ data.shape[k:r-1]
	const uint Idx = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;

	if(Idx < Num)
	{
		uint ReplacementIdx = Idx / SliceSize;
		uint ElemIdx = Idx % SliceSize;

		uint OutputIterator[NUM_DIMENSIONS];

		// Set the index of the element within the slice
		FromIdxToIterator(ElemIdx, DataTensorInfo, OutputIterator);

		// Then retrieve and set the index of the slice
		for (uint DimIdx = 0; DimIdx < PartialIndexRank; ++DimIdx)
		{
			uint IndicesAccessIdx = ReplacementIdx * PartialIndexRank + DimIdx;
			int RawIdx = CastInt64ToInt32( asuint(READ(InputIndices[IndicesAccessIdx * 2])), READ(InputIndices[IndicesAccessIdx * 2 + 1]) );
			OutputIterator[DimIdx] = NormalizeIdx(RawIdx, DataTensorInfo[DimIdx][SIZE_IDX]);
		}

		uint OutputIdx = FromIteratorToIdx(DataTensorInfo, OutputIterator);
		
		ApplyReduction(OutputIdx, Idx);
	}
}
