// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;

#define INPUT_STRIDE 0
#define OUTPUT_STRIDE 1
#define INPUT_STARTS 2
#define INPUT_STEPS 3

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Slice(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint InputIndex = 0;
		uint Offset = Index;

		for (uint dim = 0; dim < NUM_DIMENSIONS; ++dim)
		{
			uint OutDimIdx, R;
			DivMod(Offset, TensorInfo[dim][OUTPUT_STRIDE], OutDimIdx, R);
			uint InDimIdx = (uint) ((int) TensorInfo[dim][INPUT_STARTS] + (int) OutDimIdx * asint(TensorInfo[dim][INPUT_STEPS]));
			InputIndex += TensorInfo[dim][INPUT_STRIDE] * InDimIdx;
			Offset = R;
		}

		float X = Input[InputIndex];
		Output[Index] = X;
	}
}
