// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define WORK_TYPE float
#define BUFFER_TYPE float
#define READ(x) x
#define WRITE(x) x

Buffer<WORK_TYPE> Input;
Buffer<WORK_TYPE> InputSumExp;
RWBuffer<WORK_TYPE> Output;
uint Num;
uint ThreadCountX;
uint AxisSize;
uint AfterAxisSize;

#define SOFTMAX_TYPE 0
#define LOG_SOFTMAX_TYPE 1

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Softmax(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	if (Index < Num)
	{
		uint SumExpIndex;
#if SINGLE_DIMENSION == 0
		SumExpIndex = Index / AxisSize;
#else
		SumExpIndex = (Index / (AfterAxisSize * AxisSize)) * AfterAxisSize + Index % AfterAxisSize;
#endif
		WORK_TYPE SumExp = READ(InputSumExp[SumExpIndex]);
		WORK_TYPE Result = WRITE(exp(READ(Input[Index])) / SumExp);

		#if (SOFTMAX_OPERATOR_TYPE == LOG_SOFTMAX_TYPE)
			Result = log(Result);
		#endif

		Output[Index] = Result;
	}
}