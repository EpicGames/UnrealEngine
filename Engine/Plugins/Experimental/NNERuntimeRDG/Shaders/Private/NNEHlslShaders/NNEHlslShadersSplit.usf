// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input;

// Buffer array is not currently available, requires a workaround. Also can't write into more than 8 output buffers.
// RWBuffer<float> Output[MAX_NUM_SPLITS];
#define TEMPLATE_BUFFERS(TPrefix) \
	TPrefix##_0;\
	TPrefix##_1;\
	TPrefix##_2;\
	TPrefix##_3;\
	TPrefix##_4;\
	TPrefix##_5;\
	TPrefix##_6;\
	TPrefix##_7;

TEMPLATE_BUFFERS(RWBuffer<float> Output)

uint4 InputTensorInfo[MAX_NUM_DIMENSIONS];
uint4 OutputTensorInfo[MAX_NUM_SPLITS * MAX_NUM_DIMENSIONS];

uint Num;
uint ThreadCountX;

#define STRIDE_IDX 0
#define SIZE_IDX 1

#define STATIC_LOOP(Var, From, To) \
[unroll] \
for(uint Var = From; Var < To; ++Var)

void GetInputDimIterator(const uint GlobalIdx, out uint DimIterator[RANK])
{
	uint Offset = GlobalIdx;
	STATIC_LOOP(DimIdx, 0, RANK)
	{
		uint Remainder;
		DivMod(Offset, InputTensorInfo[DimIdx][STRIDE_IDX], DimIterator[DimIdx], Remainder);
		Offset = Remainder;
	}
}

struct FOutputIterator
{
	uint TensorIdx;
	uint AxisIterator;
};

FOutputIterator GetOutputIterator(const uint InputAxisIterator)
{
	FOutputIterator OutputIterator;
	uint SplitIndex = 0;
	uint PrevSplitIndex = 0;

	[loop]
	for(OutputIterator.TensorIdx = 0; OutputIterator.TensorIdx < NUM_SPLITS; ++OutputIterator.TensorIdx)
	{
		SplitIndex += OutputTensorInfo[OutputIterator.TensorIdx * MAX_NUM_DIMENSIONS + AXIS][SIZE_IDX];
		if(InputAxisIterator < SplitIndex)
		{
			OutputIterator.AxisIterator = InputAxisIterator - PrevSplitIndex;
			return OutputIterator;
		}
		PrevSplitIndex = SplitIndex;
	}
	
	return OutputIterator;
}

uint GetOutputGlobalIdx(const uint DimIterator[RANK], const uint OutputTensorIdx)
{
	uint Offset = 0;
	STATIC_LOOP(DimIdx, 0, RANK)
	{
		Offset += DimIterator[DimIdx] * OutputTensorInfo[OutputTensorIdx * MAX_NUM_DIMENSIONS + DimIdx][STRIDE_IDX];
	}
	return Offset;
}

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Split(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint DimIterator[RANK];

		GetInputDimIterator(Index, DimIterator);
		
		FOutputIterator OutputIterator = GetOutputIterator(DimIterator[AXIS]);
		DimIterator[AXIS] = OutputIterator.AxisIterator;

		// The following implements:
		// Output[OutputIterator.TensorIdx][GetOutputGlobalIdx(DimIterator, OutputIterator.TensorIdx)] = Input[Index];

		#define WRITE_BUFFER_ARRAY(Idx) \
		if(OutputIterator.TensorIdx == Idx) \
		{ \
			Output_##Idx[GetOutputGlobalIdx(DimIterator, Idx)] = Input[Index]; \
		}
		#if 0 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(0)
		#endif
		#if 1 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(1)
		#endif
		#if 2 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(2)
		#endif
		#if 3 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(3)
		#endif
		#if 4 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(4)
		#endif
		#if 5 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(5)
		#endif
		#if 6 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(6)
		#endif
		#if 7 < NUM_SPLITS
			WRITE_BUFFER_ARRAY(7)
		#endif
	}
}
