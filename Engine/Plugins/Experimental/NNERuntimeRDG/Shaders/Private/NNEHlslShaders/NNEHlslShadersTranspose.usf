// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;

#define OUTPUT_STRIDE 0
#define INPUT_STRIDE_TRANSPOSED 2

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Transpose(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (Index < Num)
	{
		uint InputIndex = 0;
		uint Offset = Index;

		for (uint dim = 0; dim < NUM_DIMENSIONS; ++dim)
		{
			uint OutDimIdx, R;
			DivMod(Offset, TensorInfo[dim][OUTPUT_STRIDE], OutDimIdx, R);
			InputIndex += TensorInfo[dim][INPUT_STRIDE_TRANSPOSED] * OutDimIdx;
			Offset = R;
		}

		float X = Input[InputIndex];
		Output[Index] = X;
	}
}
