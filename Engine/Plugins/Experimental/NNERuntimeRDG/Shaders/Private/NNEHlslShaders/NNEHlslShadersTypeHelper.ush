// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//Fetches the low 32 bits of the 64 bit two's complement number at Index and ignores the upper 32 bits. 
//This works, assuming the value is representable as a 32 bit number, since 
//expanding a 32 bit value to 64 bit involves copying the sign bit to the upper 32 bits and
//therefore bit 63 == bit 31
int GetInt32IndexFromBuffer(Buffer<int> IndexBuffer, int Index)
{
#if HAS_64BIT_INDICES
	return IndexBuffer[Index * 2];
#else
	return IndexBuffer[Index];
#endif
}
