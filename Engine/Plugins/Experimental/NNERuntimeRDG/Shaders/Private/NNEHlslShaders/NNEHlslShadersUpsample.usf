// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "NNEHlslShadersBroadcastHelper.ush"

Buffer<float> Input;
RWBuffer<float> Output;
uint4 TensorInfo[NUM_DIMENSIONS];
uint Num;
uint ThreadCountX;

#define INPUT_STRIDE 0
#define OUTPUT_STRIDE 1
#define INPUT_SIZE 2
#define SCALES_DIV 3

#if MODE == 0 // Nearest

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Upsample(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint OutputIndex = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (OutputIndex < Num)
	{
		uint InputIndex = 0;

		uint CurrIndex = OutputIndex;
		for (uint dim = 0; dim < NUM_DIMENSIONS; ++dim)
		{
			uint OutputCoord, OutputOffset;
			DivMod(CurrIndex, TensorInfo[dim][OUTPUT_STRIDE], OutputCoord, OutputOffset);

			uint InputCoord, InputOffset;
			DivMod(OutputCoord, TensorInfo[dim][SCALES_DIV], InputCoord, InputOffset);
			
			InputIndex += TensorInfo[dim][INPUT_STRIDE] * InputCoord;
			CurrIndex = OutputOffset;
		}

		Output[OutputIndex] = Input[InputIndex];
	}
}

#elif MODE == 1 // Bilinear

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Upsample(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint OutputIndex = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (OutputIndex < Num)
	{
		uint InputIndex = 0;

		uint CurrIndex = OutputIndex;
		for (uint dim = 0; dim < NUM_DIMENSIONS - 2; ++dim)
		{
			uint OutputCoord, OutputOffset;
			DivMod(CurrIndex, TensorInfo[dim][OUTPUT_STRIDE], OutputCoord, OutputOffset);

			uint InputCoord, InputOffset;
			DivMod(OutputCoord, TensorInfo[dim][SCALES_DIV], InputCoord, InputOffset);
			
			InputIndex += TensorInfo[dim][INPUT_STRIDE] * InputCoord;
			CurrIndex = OutputOffset;
		}

		const uint dim0 = NUM_DIMENSIONS - 2;
		const uint dim1 = NUM_DIMENSIONS - 1;

		uint OutputCoord0, OutputCoord1;
		DivMod(CurrIndex, TensorInfo[dim0][OUTPUT_STRIDE], OutputCoord0, OutputCoord1);

		uint InputCoord0, InputCoord1, InputOffset0, InputOffset1;
		DivMod(OutputCoord0, TensorInfo[dim0][SCALES_DIV], InputCoord0, InputOffset0);
		DivMod(OutputCoord1, TensorInfo[dim1][SCALES_DIV], InputCoord1, InputOffset1);

		InputIndex += TensorInfo[dim0][INPUT_STRIDE] * InputCoord0 + InputCoord1;

		const float x00 = Input[InputIndex];
		float x10, x01, x11;

		bool DimEnd0 = false;
		if (InputCoord0 == TensorInfo[dim0][INPUT_SIZE] - 1)
		{
			x01 = x00;
			DimEnd0 = true;
		}
		else
		{
			x01 = Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE]];
		}

		if (InputCoord1 == TensorInfo[dim1][INPUT_SIZE] - 1)
		{
			x10 = x00;
			x11 = x01;
		}
		else
		{
			x10 = Input[InputIndex + 1];
			x11 = DimEnd0 ? x10 : Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE] + 1];
		}

		float y0 = x00 + (float)InputOffset0 * (x01 - x00) / (float)TensorInfo[dim0][SCALES_DIV];
		float y1 = x10 + (float)InputOffset0 * (x11 - x10) / (float)TensorInfo[dim0][SCALES_DIV];

		Output[OutputIndex] = y0 + (float)InputOffset1 * (y1 - y0) / (float)TensorInfo[dim1][SCALES_DIV];
	}
}

#else // Trilinear

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void Upsample(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const uint OutputIndex = DispatchThreadID.y * ThreadCountX + DispatchThreadID.x;
	
	if (OutputIndex < Num)
	{
		uint InputIndex = 0;

		uint CurrIndex = OutputIndex;
		for (uint dim = 0; dim < NUM_DIMENSIONS - 3; ++dim)
		{
			uint OutputCoord, OutputOffset;
			DivMod(CurrIndex, TensorInfo[dim][OUTPUT_STRIDE], OutputCoord, OutputOffset);

			uint InputCoord, InputOffset;
			DivMod(OutputCoord, TensorInfo[dim][SCALES_DIV], InputCoord, InputOffset);
			
			InputIndex += TensorInfo[dim][INPUT_STRIDE] * InputCoord;
			CurrIndex = OutputOffset;
		}

		const uint dim0 = NUM_DIMENSIONS - 3;
		const uint dim1 = NUM_DIMENSIONS - 2;
		const uint dim2 = NUM_DIMENSIONS - 1;

		uint OutputCoord0, Mod, OutputCoord1, OutputCoord2;
		DivMod(CurrIndex, TensorInfo[dim0][OUTPUT_STRIDE], OutputCoord0, Mod);
		DivMod(Mod, TensorInfo[dim0][OUTPUT_STRIDE], OutputCoord1, OutputCoord2);

		uint InputCoord0, InputCoord1, InputCoord2, InputOffset0, InputOffset1, InputOffset2;
		DivMod(OutputCoord0, TensorInfo[dim0][SCALES_DIV], InputCoord0, InputOffset0);
		DivMod(OutputCoord1, TensorInfo[dim1][SCALES_DIV], InputCoord1, InputOffset1);
		DivMod(OutputCoord2, TensorInfo[dim2][SCALES_DIV], InputCoord2, InputOffset2);

		InputIndex += TensorInfo[dim0][INPUT_STRIDE] * InputCoord0 +
			TensorInfo[dim1][INPUT_STRIDE] * InputCoord1 +
			InputCoord2;

		const float x000 = Input[InputIndex];
		float x010, x001, x011;
		float x100, x110, x101, x111;

		bool DimEnd0 = false;
		if (InputCoord0 == TensorInfo[dim0][INPUT_SIZE] - 1)
		{
			x001 = x000;
			DimEnd0 = true;
		}
		else
		{
			x001 = Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE]];
		}

		bool DimEnd1 = false;
		if (InputCoord1 == TensorInfo[dim1][INPUT_SIZE] - 1)
		{
			x010 = x000;
			x011 = x001;
			DimEnd1 = true;
		}
		else
		{
			x010 = Input[InputIndex + TensorInfo[dim1][INPUT_STRIDE]];
			x011 = DimEnd0 ? x010 : Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE] + TensorInfo[dim1][INPUT_STRIDE]];
		}

		if (InputCoord2 == TensorInfo[dim2][INPUT_SIZE] - 1)
		{
			x100 = x000;
			x101 = x001;
			x110 = x010;
			x111 = x011;
		}
		else
		{
			x100 = Input[InputIndex + 1];
			x101 = DimEnd0 ? x100 : Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE] + 1];
			x110 = DimEnd1 ? x100 : Input[InputIndex + TensorInfo[dim1][INPUT_STRIDE] + 1];
			x111 = DimEnd0 && DimEnd1 ? x100 : DimEnd0 ? x110 : DimEnd1 ? x101 : Input[InputIndex + TensorInfo[dim0][INPUT_STRIDE] + TensorInfo[dim1][INPUT_STRIDE] + 1];
		}

		float y00 = x000 + (float)InputOffset0 * (x001 - x000) / (float)TensorInfo[dim0][SCALES_DIV];
		float y01 = x010 + (float)InputOffset0 * (x011 - x010) / (float)TensorInfo[dim0][SCALES_DIV];
		float y10 = x100 + (float)InputOffset0 * (x101 - x100) / (float)TensorInfo[dim0][SCALES_DIV];
		float y11 = x110 + (float)InputOffset0 * (x111 - x110) / (float)TensorInfo[dim0][SCALES_DIV];

		float z0 = y00 + (float)InputOffset1 * (y01 - y00) / (float)TensorInfo[dim1][SCALES_DIV];
		float z1 = y10 + (float)InputOffset1 * (y11 - y10) / (float)TensorInfo[dim1][SCALES_DIV];

		Output[OutputIndex] = z0 + (float)InputOffset2 * (z1 - z0) / (float)TensorInfo[dim2][SCALES_DIV];
	}
}

#endif