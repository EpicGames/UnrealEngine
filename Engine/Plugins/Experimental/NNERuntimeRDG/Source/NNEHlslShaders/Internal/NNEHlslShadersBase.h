// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"

namespace UE::NNEHlslShaders::Internal
{
	class FHlslShaderBase : public FGlobalShader
	{
	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		FHlslShaderBase() {}
		FHlslShaderBase(const ShaderMetaType::CompiledShaderInitializerType& initType) : FGlobalShader(initType) {}
	};
	
} // UE::NNEHlslShaders::Internal