// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"


namespace UE::NNEHlslShaders::Internal
{
	class FBatchNormalizationConstants
	{
	public:

		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TBatchNormalizationCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TBatchNormalizationCS);
		SHADER_USE_PARAMETER_STRUCT(TBatchNormalizationCS, FHlslShaderBase)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, X)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Scales)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Bias)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Mean)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Var)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, DimC)
			SHADER_PARAMETER(uint32, SpatialVolume)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(float, Epsilon)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
