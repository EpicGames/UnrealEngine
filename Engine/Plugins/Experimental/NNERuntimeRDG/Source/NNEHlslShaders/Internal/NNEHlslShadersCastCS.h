// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersTypeHelper.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FCastConstants
	{
	public:
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TCastCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TCastCS);
		SHADER_USE_PARAMETER_STRUCT(TCastCS, FHlslShaderBase)

		class FInputType : SHADER_PERMUTATION_ENUM_CLASS("INPUT_TYPE_ENUM", ENNEShaderDataType);
		class FOutputType : SHADER_PERMUTATION_ENUM_CLASS("OUTPUT_TYPE_ENUM", ENNEShaderDataType);
		using FPermutationDomain = TShaderPermutationDomain<FInputType,FOutputType>;

	public:
		// Depending on the shader permutation 'Input' and 'Output' buffers have
		// different element types and are not only used with uint values.
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, Output)
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
