// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class EConvAlgorithm : uint8
	{
		SharedMemory = 0,
		MAX
	};

	enum class EConvGroupSize : uint8
	{
		Size128 = 0,
		Size256,
		Size512,
		MAX
	};

	enum class EConvAutoPad : uint8
	{
		NOTSET = 0,// Use pad values passed in the array
		SAME_UPPER,// Auto-pad to match input and output shape with potetnial extra padding at the end
		SAME_LOWER,// Auto-pad to match input and output shape with potetnial extra padding at the beginning
		VALID,// Set all paddings to zero
		MAX
	};

	class FConvConstants
	{
	public:
		static const int32 MAX_NUM_DIMENSIONS{4};
		static const int32 MIN_NUM_READS_PER_THREAD_POW2{1};
		static const int32 MAX_NUM_READS_PER_THREAD_POW2{4};
	};

	class NNEHLSLSHADERS_API FConvCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvCS);
		SHADER_USE_PARAMETER_STRUCT(FConvCS, FHlslShaderBase)

		class FConvAlgorithm : SHADER_PERMUTATION_ENUM_CLASS("ALGORITHM", EConvAlgorithm);
		class FConvAreWeightsTransposed : SHADER_PERMUTATION_BOOL("WEIGHTS_TRANSPOSED");
		class FConvGroupSize : SHADER_PERMUTATION_ENUM_CLASS("GROUP_SIZE", EConvGroupSize);
		class FConvNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FConvConstants::MAX_NUM_DIMENSIONS);
		class FConvNumReadsPerThread : SHADER_PERMUTATION_RANGE_INT("NUM_READS_PER_THREAD_POW2", FConvConstants::MIN_NUM_READS_PER_THREAD_POW2, FConvConstants::MAX_NUM_READS_PER_THREAD_POW2);
		class FConvHasB : SHADER_PERMUTATION_BOOL("HAS_B");
		using FPermutationDomain = TShaderPermutationDomain<FConvAlgorithm, FConvAreWeightsTransposed, FConvGroupSize, FConvNumDimensions, FConvNumReadsPerThread, FConvHasB>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, X)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, W)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Y)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, B)
			SHADER_PARAMETER_ARRAY(FIntVector4, Dilation_Stride_XBlockStartOffset_DilationXBlockStride, [FConvConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride, [FConvConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, YDimension_YMemoryStride_XDimension_XMemoryStride, [FConvConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride, [FConvConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FVector4f, OneDiv_GroupStride_GroupThreadStride_XBlockStride, [FConvConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(int32, NumWChannels)
			SHADER_PARAMETER(int32, NumWFeatures)
			SHADER_PARAMETER(int32, YBatchStride)
			SHADER_PARAMETER(int32, YOutputKernelStride)
			SHADER_PARAMETER(int32, XBatchStride)
			SHADER_PARAMETER(int32, XChannelStride)
			SHADER_PARAMETER(int32, XBlockSize)
			SHADER_PARAMETER(int32, NumChannelBatches)
			SHADER_PARAMETER(int32, NumChannelsPerBatch)
			SHADER_PARAMETER(int32, WOutputKernelStride)
			SHADER_PARAMETER(int32, WChannelBatchSize)
			SHADER_PARAMETER(int32, WChannelSize)
			SHADER_PARAMETER(float, GroupsDivM)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

	public:
		static TArray<int32> GetOutputShape(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads);

		static void FillInParameters(EConvGroupSize GroupSize, TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, bool HasB,
				EConvAutoPad AutoPad, int Group, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, FConvCS::FParameters& Parameters);

		static int32 GetNumReadsPerThread(EConvGroupSize GroupSize, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides);

		/**
		* @brief Computes the group shape such that all dimension have roughly equal sizes.
		*
		* @param GroupSize The enum indicating the number of threads contained by a single group.
		* @param NumDimensions The number of dimensions.
		* @return TArray<int32> An array of size \p NumDimensions containing the number of threads in each dimension to form a volume of a total number of threads indicated by \p GroupSize
		*/
		static TArray<int32> GetGroupShape(EConvGroupSize GroupSize, int32 NumDimensions);

		/**
		* @brief Get the group count vector used to launch the gpu shader thread groups
		*
		* @param YShape The shape of the output as computed by GetOutputShape()
		* @param YShape The shape of a single thread group as computed by GetGroupShape()
		* @return FIntVector The number of thread groups to instantiate. z corresponds to the batch and y to the output kernel.
		*/
		static FIntVector GetGroupCount(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape);

		static EConvGroupSize GetBiggestCompatibleGroupSize(TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides);
		static EConvGroupSize GetMinimalGroupSize(TArrayView<const int32> WShape);

		static TArray<int32> GetPadding(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads);

		static void LexFromString(EConvAutoPad& OutValue, const TCHAR* StringVal);
	};
} // UE::NNEHlslShaders::Internal