// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class NNEHLSLSHADERS_API FConvMatmulCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvMatmulCS);
		SHADER_USE_PARAMETER_STRUCT(FConvMatmulCS, FHlslShaderBase)

		class FConvMatmulAreWeightsTransposed : SHADER_PERMUTATION_BOOL("WEIGHTS_TRANSPOSED");
		class FConvMatmulHasBias : SHADER_PERMUTATION_BOOL("HAS_BIAS");
		using FPermutationDomain = TShaderPermutationDomain<FConvMatmulAreWeightsTransposed, FConvMatmulHasBias>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Weight)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Bias)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(int32, Ci)
			SHADER_PARAMETER(int32, Hi)
			SHADER_PARAMETER(int32, Wi)
			SHADER_PARAMETER(int32, Cw)
			SHADER_PARAMETER(int32, Hw)
			SHADER_PARAMETER(int32, Ww)
			SHADER_PARAMETER(int32, Ho)
			SHADER_PARAMETER(int32, Wo)
			SHADER_PARAMETER(int32, StrideH)
			SHADER_PARAMETER(int32, StrideW)
			SHADER_PARAMETER(int32, PadLeft)
			SHADER_PARAMETER(int32, PadTop)
		END_SHADER_PARAMETER_STRUCT()

		static FIntVector GetGroupCount(TConstArrayView<uint32> OutputShape);
	};
} // UE::NNEHlslShaders::Internal