// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class EConvTransposeAlgorithm : uint8
	{
		SharedMemory = 0,
		MAX
	};

	enum class EConvTransposeGroupSize : uint8
	{
		Size128 = 0,
		Size256,
		Size512,
		MAX
	};

	enum class EConvTransposeAutoPad : uint8
	{
		NOTSET = 0,// Use pad values passed in the array
		SAME_UPPER,// Auto-pad to match input and output shape with potetnial extra padding at the end
		SAME_LOWER,// Auto-pad to match input and output shape with potetnial extra padding at the beginning
		VALID,// Set all paddings to zero
		MAX
	};

	class FConvTransposeConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{4};
		static const int32 MIN_NUM_READS_PER_THREAD_POW2{1};
		static const int32 MAX_NUM_READS_PER_THREAD_POW2{3};
	};

	class NNEHLSLSHADERS_API FConvTransposeCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvTransposeCS);
		SHADER_USE_PARAMETER_STRUCT(FConvTransposeCS, FHlslShaderBase);

		class FConvTransposeAlgorithm : SHADER_PERMUTATION_ENUM_CLASS("ALGORITHM", EConvTransposeAlgorithm);
		class FConvTransposeGroupSize : SHADER_PERMUTATION_ENUM_CLASS("GROUP_SIZE", EConvTransposeGroupSize);
		class FConvTransposeNumStackDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_STACK_DIMENSIONS", 1, FConvTransposeConstants::MAX_NUM_DIMENSIONS);
		class FConvTransposeNumReadsPerThread : SHADER_PERMUTATION_RANGE_INT("NUM_READS_PER_THREAD_POW2", FConvTransposeConstants::MIN_NUM_READS_PER_THREAD_POW2, FConvTransposeConstants::MAX_NUM_READS_PER_THREAD_POW2);
		class FConvTransposeHasB : SHADER_PERMUTATION_BOOL("HAS_B");
		using FPermutationDomain = TShaderPermutationDomain<FConvTransposeAlgorithm, FConvTransposeGroupSize, FConvTransposeNumStackDimensions, FConvTransposeNumReadsPerThread, FConvTransposeHasB>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, X)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, W)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Y)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, B)
			SHADER_PARAMETER_ARRAY(FIntVector4, Dilation_Stride_XBlockStartOffset_DilationXBlockStride, [FConvTransposeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride, [FConvTransposeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, YDimension_YMemoryStride_XDimension_XMemoryStride, [FConvTransposeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FIntVector4, XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride, [FConvTransposeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FVector4f, OneDiv_GroupStride_GroupThreadStride_OneDivStride, [FConvTransposeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(int32, NumWChannels)
			SHADER_PARAMETER(int32, NumOutChannelsDivGroup)
			SHADER_PARAMETER(int32, YBatchStride)
			SHADER_PARAMETER(int32, YOutputKernelStride)
			SHADER_PARAMETER(int32, XBatchStride)
			SHADER_PARAMETER(int32, XChannelStride)
			SHADER_PARAMETER(int32, XBlockSize)
			SHADER_PARAMETER(int32, NumChannelBatches)
			SHADER_PARAMETER(int32, NumChannelsPerBatch)
			SHADER_PARAMETER(int32, WOutputKernelStride)
			SHADER_PARAMETER(int32, WChannelBatchSize)
			SHADER_PARAMETER(int32, WChannelSize)
			SHADER_PARAMETER(float, GroupsDivM)
			SHADER_PARAMETER(float, OneDivGroup)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static TArray<int32> GetOutputShape(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvTransposeAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding, int32 Group);

		static void FillInParameters(EConvTransposeGroupSize GroupSize, TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, bool HasB, EConvTransposeAutoPad AutoPad, int32 Group, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding, FConvTransposeCS::FParameters& Parameters);

		static int32 GetNumReadsPerThread(EConvTransposeGroupSize GroupSize, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides);

		static TArray<int32> GetGroupShape(EConvTransposeGroupSize GroupSize, int32 NumDimensions);

		static FIntVector GetGroupCount(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape);

		static EConvTransposeGroupSize GetMinimalGroupSize(TArrayView<const uint32> WShape);

		static void LexFromString(EConvTransposeAutoPad& OutValue, const TCHAR* StringVal);
		
	private:

		static TArray<int32> GetXBlockShape(TArrayView<const int32> GroupShape, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides);

		static TArray<int32> GetPadding(TArrayView<const uint32> WShape, EConvTransposeAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding);

		static int32 GetNumThreadsPerGroup(EConvTransposeGroupSize GroupSize);

		static TArray<int32> GetGridShape(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape);
	};
} // UE::NNEHlslShaders::Internal