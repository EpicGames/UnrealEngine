// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FConvWinogradInputConstants
	{
	public:

		static const int32 THREADGROUP_SIZE_X{ 32 };
	};

	class NNEHLSLSHADERS_API FConvWinogradInputCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvWinogradInputCS);
		SHADER_USE_PARAMETER_STRUCT(FConvWinogradInputCS, FHlslShaderBase)

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(int32, C)
			SHADER_PARAMETER(int32, H)
			SHADER_PARAMETER(int32, W)
			SHADER_PARAMETER(int32, WBlockCount)
			SHADER_PARAMETER(int32, CInputStride)
			SHADER_PARAMETER(int32, HInputStride)
			SHADER_PARAMETER(int32, NiOutputStride)
			SHADER_PARAMETER(int32, MatrixOutputStride)
			SHADER_PARAMETER(int32, COutputStride)
			SHADER_PARAMETER(int32, HOutputStride)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
