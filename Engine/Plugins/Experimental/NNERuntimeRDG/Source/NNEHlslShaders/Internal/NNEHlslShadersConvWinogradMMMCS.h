// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersTypeHelper.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class NNEHLSLSHADERS_API FConvWinogradMMMCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvWinogradMMMCS);
		SHADER_USE_PARAMETER_STRUCT(FConvWinogradMMMCS, FHlslShaderBase)

		class FDataType : SHADER_PERMUTATION_ENUM_CLASS("DATA_TYPE_ENUM", ENNEShaderDataType);
		class FBlockSizeN : SHADER_PERMUTATION_SPARSE_INT("BLOCK_ELEM_COUNT_N", 16, 32, 64);
		using FPermutationDomain = TShaderPermutationDomain<FDataType, FBlockSizeN>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Weight)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(int32, M)
			SHADER_PARAMETER(int32, N)
			SHADER_PARAMETER(int32, K)
			SHADER_PARAMETER(int32, MatrixInputStride)
			SHADER_PARAMETER(int32, KInputStride)
			SHADER_PARAMETER(int32, MatrixWeightStride)
			SHADER_PARAMETER(int32, KWeightStride)
			SHADER_PARAMETER(int32, MatrixOutputStride)
			SHADER_PARAMETER(int32, NOutputStride)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static int GetOptimalBlockSizeN(int M, int K, int N);
	};
} // UE::NNEHlslShaders::Internal
