// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersTypeHelper.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FConvWinogradOutputConstants
	{
	public:

		static const int32 THREADGROUP_SIZE_X{ 32 };
	};

	class NNEHLSLSHADERS_API FConvWinogradOutputCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvWinogradOutputCS);
		SHADER_USE_PARAMETER_STRUCT(FConvWinogradOutputCS, FHlslShaderBase)

		class FHasBias : SHADER_PERMUTATION_BOOL("HAS_BIAS");
		class FDataType : SHADER_PERMUTATION_ENUM_CLASS("DATA_TYPE_ENUM", ENNEShaderDataType);
		using FPermutationDomain = TShaderPermutationDomain<FHasBias, FDataType>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Bias)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(int32, C)
			SHADER_PARAMETER(int32, H)
			SHADER_PARAMETER(int32, W)
			SHADER_PARAMETER(int32, WBlockCount)
			SHADER_PARAMETER(int32, NiInputStride)
			SHADER_PARAMETER(int32, MatrixInputStride)
			SHADER_PARAMETER(int32, CInputStride)
			SHADER_PARAMETER(int32, HInputStride)
			SHADER_PARAMETER(int32, COutputStride)
			SHADER_PARAMETER(int32, HOutputStride)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
