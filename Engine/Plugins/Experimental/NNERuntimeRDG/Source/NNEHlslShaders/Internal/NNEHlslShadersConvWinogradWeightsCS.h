// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FConvWinogradWeightsConstants
	{
	public:

		static const int32 THREADGROUP_SIZE_X{ 32 };
	};

	class NNEHLSLSHADERS_API FConvWinogradWeightsCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FConvWinogradWeightsCS);
		SHADER_USE_PARAMETER_STRUCT(FConvWinogradWeightsCS, FHlslShaderBase)

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(int32, Ci)
			SHADER_PARAMETER(int32, Cw)
			SHADER_PARAMETER(int32, CwInputStride)
			SHADER_PARAMETER(int32, MatrixOutputStride)
			SHADER_PARAMETER(int32, CiOutputStride)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
