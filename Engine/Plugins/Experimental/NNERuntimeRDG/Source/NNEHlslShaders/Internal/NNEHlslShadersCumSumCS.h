// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNEHlslShadersBase.h"
#include "RenderGraphFwd.h"
#include "RenderGraphUtils.h"
#include "RHIGlobals.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE::Internal { class FTensor; }

namespace UE::NNEHlslShaders::Internal
{

	class FCumSumConstants
	{
	public:
		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 THREADGROUP_SIZE{ 256 };
		static const int32 VALUES_PER_THREAD{ 12 };
		static const int32 INIT_THREADGROUP_SIZE{ 768 };
	};

	struct FPartitionDescriptor
	{
		int32 StatusFlag;
		float Aggregate;
		float InclusivePrefix;
		int32 PadToQWord;
	};

	class NNEHLSLSHADERS_API TInitCumSumCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TInitCumSumCS);
		SHADER_USE_PARAMETER_STRUCT(TInitCumSumCS, FHlslShaderBase)

		using FPermutationDomain = TShaderPermutationDomain<>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, NumThreadGroupsPerScan)
			SHADER_PARAMETER(uint32, NumThreadGroupsY)
			SHADER_PARAMETER(uint32, NumThreadGroupsZ)
			SHADER_PARAMETER(uint32, NumInitThreadGroups)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, GlobalPartitionIndex)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPartitionDescriptor>, PartitionDescriptor)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};

	class NNEHLSLSHADERS_API TCumSumCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TCumSumCS);
		SHADER_USE_PARAMETER_STRUCT(TCumSumCS, FHlslShaderBase)

		using FPermutationDomain = TShaderPermutationDomain<>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, NumThreadGroupsPerScan)
			SHADER_PARAMETER(uint32, NumThreadGroupsY)
			SHADER_PARAMETER(uint32, NumThreadGroupsZ)
			SHADER_PARAMETER(uint32, NumScanValues)
			SHADER_PARAMETER(uint32, Axis)
			SHADER_PARAMETER(uint32, AxisStride)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, TensorInfo, [FCumSumConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, GlobalPartitionIndex)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPartitionDescriptor>, PartitionDescriptor)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal