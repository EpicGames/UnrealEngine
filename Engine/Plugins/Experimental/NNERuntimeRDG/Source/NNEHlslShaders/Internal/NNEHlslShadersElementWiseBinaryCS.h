// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersOperator.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FElementWiseBinaryConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TElementWiseBinaryCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TElementWiseBinaryCS);
		SHADER_USE_PARAMETER_STRUCT(TElementWiseBinaryCS, FHlslShaderBase)

		class FOperatorType : SHADER_PERMUTATION_ENUM_CLASS("OP_TYPENAME", EElementWiseBinaryOperatorType);
		class FBinaryNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FElementWiseBinaryConstants::MAX_NUM_DIMENSIONS);
		using FPermutationDomain = TShaderPermutationDomain<FOperatorType, FBinaryNumDimensions>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, LHSInput)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, RHSInput)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, TensorInfo, [FElementWiseBinaryConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

	private:

		static const FString GetOpFunc(EElementWiseBinaryOperatorType OpType);
	};
} // UE::NNEHlslShaders::Internal
