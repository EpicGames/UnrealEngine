// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersOperator.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FElementWiseUnaryConstants
	{
	public:

		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TElementWiseUnaryCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TElementWiseUnaryCS);
		SHADER_USE_PARAMETER_STRUCT(TElementWiseUnaryCS, FHlslShaderBase)

		class FOperatorType : SHADER_PERMUTATION_ENUM_CLASS("OP_TYPENAME", EElementWiseUnaryOperatorType);
		class FAlphaOnGPU : SHADER_PERMUTATION_BOOL("ALPHA_ON_GPU");
		class FBetaOnGPU : SHADER_PERMUTATION_BOOL("BETA_ON_GPU");
		using FPermutationDomain = TShaderPermutationDomain<FOperatorType,FAlphaOnGPU,FBetaOnGPU>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(float, Alpha)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, AlphaTensor)
			SHADER_PARAMETER(float, Beta)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, BetaTensor)
			SHADER_PARAMETER(float, Gamma)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

	private:

		static const FString GetOpFunc(EElementWiseUnaryOperatorType OpType);
	};
} // UE::NNEHlslShaders::Internal
