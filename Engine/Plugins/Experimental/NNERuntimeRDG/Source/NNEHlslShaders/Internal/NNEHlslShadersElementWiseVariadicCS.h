// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "NNEHlslShadersOperator.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FElementWiseVariadicConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const uint32 NUM_GROUP_THREADS{ 256 };
		static const uint32 MAX_NUM_INPUT{ 4 };
	};

	class NNEHLSLSHADERS_API TElementWiseVariadicCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TElementWiseVariadicCS);
		SHADER_USE_PARAMETER_STRUCT(TElementWiseVariadicCS, FHlslShaderBase)

		class FOperatorType : SHADER_PERMUTATION_ENUM_CLASS("OP_TYPENAME", EElementWiseVariadicOperatorType);
		class FApplyScale : SHADER_PERMUTATION_BOOL("APPLYSCALE");
		class FOutputAsInput : SHADER_PERMUTATION_BOOL("OUTPUTASINPUT");
		class FNumInput : SHADER_PERMUTATION_RANGE_INT("NUMINPUT", 1, FElementWiseVariadicConstants::MAX_NUM_INPUT);
		class FVariadicNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FElementWiseVariadicConstants::MAX_NUM_DIMENSIONS);
		using FPermutationDomain = TShaderPermutationDomain<FOperatorType, FApplyScale, FOutputAsInput, FNumInput, FVariadicNumDimensions>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input0)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input1)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input2)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input3)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, InputTensorInfo, [FElementWiseVariadicConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FUintVector4, OutputTensorInfo, [FElementWiseVariadicConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(float, Scale)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

	private:

		static const FString GetOpFunc(EElementWiseVariadicOperatorType OpType);
	};
} // UE::NNEHlslShaders::Internal