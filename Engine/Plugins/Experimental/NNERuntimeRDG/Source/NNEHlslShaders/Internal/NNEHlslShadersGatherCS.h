// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE { class FTensorShape; }

namespace UE::NNEHlslShaders::Internal
{
	class FGatherConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	// template <typename DataElementType, typename IndicesElementType>
	class NNEHLSLSHADERS_API TGatherCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TGatherCS);
		SHADER_USE_PARAMETER_STRUCT(TGatherCS, FHlslShaderBase)

		class FGatherNumOutputDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_OUTPUT_DIMENSIONS", 1, FGatherConstants::MAX_NUM_DIMENSIONS);
		class FGather64BitIndices : SHADER_PERMUTATION_BOOL("HAS_64BIT_INDICES");
		using FPermutationDomain = TShaderPermutationDomain<FGatherNumOutputDimensions,FGather64BitIndices>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, Axis)
			SHADER_PARAMETER(int32, OutputSize)
			SHADER_PARAMETER(int32, NumDataDimensions)
			SHADER_PARAMETER(int32, NumIndicesDimensions)
			SHADER_PARAMETER_ARRAY(FIntVector4, DataStride_IndicesStride_OutputStride_DataSizes, [FGatherConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FVector4f, OneDivDataStride_OneDivIndicesStride_OneDivOutputStride, [FGatherConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Data)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int32>, Indices)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void FillInParameters(int32 Axis, const NNE::FTensorShape& Data, const NNE::FTensorShape& Indices, FParameters& Parameters);

		static FIntVector GetGroupCount(const FParameters& Parameters);
	};
} // UE::NNEHlslShaders::Internal