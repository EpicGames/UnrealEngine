// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	class FGatherElementsConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FGatherElementsCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FGatherElementsCS);
		SHADER_USE_PARAMETER_STRUCT(FGatherElementsCS, FHlslShaderBase)

		class FGatherElementsDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FGatherElementsConstants::MAX_NUM_DIMENSIONS);
		class FGatherElements64BitIndices : SHADER_PERMUTATION_BOOL("HAS_64BIT_INDICES");
		using FPermutationDomain = TShaderPermutationDomain<FGatherElementsDimensions, FGatherElements64BitIndices>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int32>, Indices)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FVector4f, OneDivOutputStrides, [FGatherElementsConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FInt32Vector4, Input_OutputStrides, [FGatherElementsConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Axis)
			SHADER_PARAMETER(uint32, AxisSize)
			SHADER_PARAMETER(uint32, OutputSize)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
