// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE { class FTensorShape; }

namespace UE::NNEHlslShaders::Internal
{
	enum class EGemmCScalar : uint8
	{
		No = 0,
		Yes,
		NoBias,
		MAX
	};

	enum class EGemmAlgorithm : uint8
	{
		Simple8x8 = 0,
		Simple16x16,
		Simple32x32,
		Simple256x1,
		SharedMemory8x8,
		SharedMemory16x16,
		SharedMemory32x32,
		MultiWrite1x16,
		MultiWrite2x16,
		MultiWrite1x32,
		MultiWrite2x32,
		MultiWrite4x32,
		MultiWrite2x64,
		MultiWrite4x64,
		MultiWrite8x64,
		MAX
	};

	class FGemmConstants
	{
	public:

		static const int32 MAX_NUM_STACK_DIMENSIONS{8};
	};

	class NNEHLSLSHADERS_API TGemmCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TGemmCS);
		SHADER_USE_PARAMETER_STRUCT(TGemmCS, FHlslShaderBase)

		class FGemmCScalar : SHADER_PERMUTATION_ENUM_CLASS("C_SCALAR", EGemmCScalar);
		class FGemmAlgorithm : SHADER_PERMUTATION_ENUM_CLASS("ALGORITHM", EGemmAlgorithm);
		class FGemmNumStackDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_STACK_DIMENSIONS", 0, FGemmConstants::MAX_NUM_STACK_DIMENSIONS);
		using FPermutationDomain = TShaderPermutationDomain<FGemmCScalar, FGemmAlgorithm, FGemmNumStackDimensions>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(float, Alpha)
			SHADER_PARAMETER(float, Beta)
			SHADER_PARAMETER(int32, TransA)
			SHADER_PARAMETER(int32, TransB)
			SHADER_PARAMETER(uint32, M)
			SHADER_PARAMETER(uint32, N)
			SHADER_PARAMETER(uint32, K)
			SHADER_PARAMETER(uint32, MxK)
			SHADER_PARAMETER(uint32, KxN)
			SHADER_PARAMETER(uint32, MxN)
			SHADER_PARAMETER(uint32, CWidth)
			SHADER_PARAMETER(uint32, CHeight)
			SHADER_PARAMETER(float, CScalar)
			SHADER_PARAMETER_ARRAY(FUint32Vector4, StackShapeA_StackShapeB_StackStrideA_StackStrideB, [FGemmConstants::MAX_NUM_STACK_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, A)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, B)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, C)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Y)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void FillInParameters(float Alpha, float Beta, int32 TransA, int32 TransB, const NNE::FTensorShape& InputA, const NNE::FTensorShape& InputB, const NNE::FTensorShape* InputC, float CScalar, FParameters& Parameters);

		static void FillInParametersMatMul(const NNE::FTensorShape& InputA, const NNE::FTensorShape& InputB, FParameters& Parameters);

		static FIntVector GetGroupCount(const FParameters& Parameters, EGemmAlgorithm Algorithm, int32 NumStackDimensions);
		static EGemmAlgorithm GetAlgorithm(const FParameters& Parameters);
	};
} // UE::NNEHlslShaders::Internal