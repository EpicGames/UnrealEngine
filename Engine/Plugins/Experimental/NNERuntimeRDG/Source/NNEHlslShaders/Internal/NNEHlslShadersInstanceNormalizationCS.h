// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE::Internal { class FTensor; }

namespace UE::NNEHlslShaders::Internal
{
	class FInstanceNormalizationConstants
	{
	public:
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TInstanceNormalizationCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TInstanceNormalizationCS);
		SHADER_USE_PARAMETER_STRUCT(TInstanceNormalizationCS, FHlslShaderBase)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(uint32, InstanceSize)
			SHADER_PARAMETER(uint32, ChannelSize)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputScale)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputBias)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputMean)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputInvStdDev)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			END_SHADER_PARAMETER_STRUCT()

			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal