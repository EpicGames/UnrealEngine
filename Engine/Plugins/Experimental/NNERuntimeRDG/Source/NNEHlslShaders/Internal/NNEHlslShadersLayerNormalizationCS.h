// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE::Internal { class FTensor; }

namespace UE::NNEHlslShaders::Internal
{
	class FLayerNormalizationConstants
	{
	public:
		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 768 };
	};

	class NNEHLSLSHADERS_API TLayerNormalizationCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TLayerNormalizationCS);
		SHADER_USE_PARAMETER_STRUCT(TLayerNormalizationCS, FHlslShaderBase)

		class FLayerNormalizationNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FLayerNormalizationConstants::MAX_NUM_DIMENSIONS);
		class FLayerNormalizationHasB : SHADER_PERMUTATION_BOOL("HAS_B");
		using FPermutationDomain = TShaderPermutationDomain<FLayerNormalizationNumDimensions, FLayerNormalizationHasB>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, Axis)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(float, Epsilon)
			SHADER_PARAMETER(uint32, LayerSize)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_ARRAY(FUintVector4, InputTensorInfo, [FLayerNormalizationConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputScale)
			SHADER_PARAMETER_ARRAY(FUintVector4, ScaleTensorInfo, [FLayerNormalizationConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputBias)
			SHADER_PARAMETER_ARRAY(FUintVector4, BiasTensorInfo, [FLayerNormalizationConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputMean)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputInvStdDev)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			END_SHADER_PARAMETER_STRUCT()

			static void FillInParameters(TConstArrayView<uint32> Shape, int32 Axis, float Epsilon, FParameters* Parameters);
			static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal