// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::NNEHlslShaders::Internal
{

	//One input element-wise operators
	enum class EElementWiseUnaryOperatorType : uint8
	{
		Abs,
		Acos,
		Acosh,
		Asin,
		Asinh,
		Atan,
		Atanh,
		//BitShift, //Note: need integer tensors
		Ceil,
		Clip,
		Cos,
		Cosh,
		Elu,
		Erf,
		Exp,
		Floor,
		IsInf,
		IsNan,
		HardSigmoid,
		HardSwish,
		LeakyRelu,
		Log,
		Neg,
		//Not,      //Note: need bool tensors
		Reciprocal,
		Relu,
		Round,
		Selu,
		Sigmoid,
		Sign,
		Sin,
		Sinh,
		Softplus,
		Softsign,
		Sqrt,
		Tan,
		Tanh,

		MAX
	};

	//Two inputs element-wise operators with multi-directional broadcast
	//see https://github.com/onnx/onnx/blob/main/docs/Broadcasting.md
	enum class EElementWiseBinaryOperatorType : uint8
	{
		Add,
		//And,           //Note: need boolean tensors
		Div,
		//Equal,         //Note: need boolean tensors
		//Greater,       //Note: need boolean tensors
		//GreaterOrEqual,//Note: need boolean tensors
		//Less,          //Note: need boolean tensors
		//LessOrEqual,   //Note: need boolean tensors
		Mod,
		Mul,
		//Or,            //Note: need boolean tensors
		Prelu,           //Note: only broadcast from slope to input.
		Pow,
		Sub,
		//Xor,           //Note:  need boolean tensors

		MAX
	};

	//Variable number of inputs element-wise operators with multi-directional broadcast
	//see https://github.com/onnx/onnx/blob/main/docs/Broadcasting.md
	enum class EElementWiseVariadicOperatorType : uint8
	{
		Max,
		Min,
		Mean,
		Sum,

		MAX
	};

} // namespace UE::NNEHlslShaders::Internal