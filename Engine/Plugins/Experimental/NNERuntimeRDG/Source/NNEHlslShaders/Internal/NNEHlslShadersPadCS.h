// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class EPadMode : uint8
	{
		CONSTANT = 0,
		REFLECT,
		EDGE,
		MAX
	};
	
	class FPadConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FPadCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FPadCS);
		SHADER_USE_PARAMETER_STRUCT(FPadCS, FHlslShaderBase)

		class FPadMode : SHADER_PERMUTATION_ENUM_CLASS("MODE", EPadMode);
		class FPadNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FPadConstants::MAX_NUM_DIMENSIONS);
		using FPermutationDomain = TShaderPermutationDomain<FPadMode, FPadNumDimensions>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, TensorInfo, [FPadConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(float, Value)
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static void LexFromString(EPadMode& OutValue, const TCHAR* StringVal);
	};
} // UE::NNEHlslShaders::Internal
