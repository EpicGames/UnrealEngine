// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class EPoolOperatorType : uint8
	{
		MAX_POOL,
		AVERAGE_POOL,
		MAX
	};

	class FPoolConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 MAX_NUM_SPATIAL_DIMENSIONS{ 6 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FPoolCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FPoolCS);
		SHADER_USE_PARAMETER_STRUCT(FPoolCS, FHlslShaderBase)

		class FPoolNumSpatialDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_SPATIAL_DIMENSIONS", 1, FPoolConstants::MAX_NUM_SPATIAL_DIMENSIONS);
		class FPoolType : SHADER_PERMUTATION_ENUM_CLASS("POOL_OPERATOR_TYPE", EPoolOperatorType);
		using FPermutationDomain = TShaderPermutationDomain<FPoolNumSpatialDimensions, FPoolType>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, TensorInfo, [FPoolConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FUintVector4, SpatialInfo, [FPoolConstants::MAX_NUM_SPATIAL_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(uint32, KernelVolume)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
