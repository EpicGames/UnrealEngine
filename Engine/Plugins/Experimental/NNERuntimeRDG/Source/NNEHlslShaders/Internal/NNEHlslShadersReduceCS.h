// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNEHlslShadersBase.h"
#include "RenderGraphFwd.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"
#include "RHIGlobals.h"

namespace UE::NNE::Internal { class FTensor; }

namespace UE::NNEHlslShaders::Internal
{
	enum class EReduceOperatorType : uint8
	{
		Average = 0,
		L1,
		L2,
		LogSumExp,
		Max,
		Min,
		Prod,
		Sum,
		SumExp,//Should not be used for multiple axis reduction
		AverageInvStdDev,//Should not be used for multiple axis reduction
		MAX
	};

	class FReduceConstants
	{
	public:
		static const int32 THREADGROUP_SIZE{ 768 };
	};

	class NNEHLSLSHADERS_API TReduceCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TReduceCS);
		SHADER_USE_PARAMETER_STRUCT(TReduceCS, FHlslShaderBase)

		class FReduceType : SHADER_PERMUTATION_ENUM_CLASS("REDUCE_OPERATOR_TYPE", EReduceOperatorType);
		using FPermutationDomain = TShaderPermutationDomain<FReduceType>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, NumElemBeforeAxis)
			SHADER_PARAMETER(int32, AxisSize)
			SHADER_PARAMETER(int32, NumElemAfterAxis)
			SHADER_PARAMETER(FUintVector4, DispatchIdxAndStride)
			SHADER_PARAMETER(float, Epsilon)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output2)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static void FillInParameters(TConstArrayView<uint32> Shape, int32 Axis, FParameters* Parameters);
		static void EnqueueRDG(FRDGBuilder& GraphBuilder, FParameters* Parameters, FRDGBufferRef Input, FRDGBufferRef Output, EReduceOperatorType OperatorType, FRDGBufferRef Output2 = {});
	};
} // UE::NNEHlslShaders::Internal