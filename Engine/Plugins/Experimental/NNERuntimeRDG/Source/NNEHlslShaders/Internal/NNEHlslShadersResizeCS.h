// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{

	enum class ECoordTransMode : uint8
	{
		HalfPixel = 0,
		HalfPixelSymmetric,
		PytorchHalfPixel,
		AlignCorners,
		Asymmetric,
		TfHalfPixelForNn,
		TfCropAndResize,
		MAX
	};

	enum class EMode : uint8
	{
		Nearest = 0,
		Linear,
		Cubic,
		MAX
	};

	enum class ENearestMode : uint8
	{
		RoundPreferFloor = 0,
		RoundPreferCeil,
		Floor,
		Ceil,
		MAX
	};

	class FResizeConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FResizeCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FResizeCS);
		SHADER_USE_PARAMETER_STRUCT(FResizeCS, FHlslShaderBase)

		class FResizeNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FResizeConstants::MAX_NUM_DIMENSIONS);
		class FMode : SHADER_PERMUTATION_ENUM_CLASS("MODE", EMode);
		class FNearestMode : SHADER_PERMUTATION_ENUM_CLASS("NEAREST_MODE", ENearestMode);
		class FCoordTransMode : SHADER_PERMUTATION_ENUM_CLASS("COORD_TRANS_MODE", ECoordTransMode);
		using FPermutationDomain = TShaderPermutationDomain<FResizeNumDimensions, FMode, FNearestMode, FCoordTransMode>;

	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_ARRAY(FUintVector4, InputTensorInfo, [FResizeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, OutputTensorInfo, [FResizeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FVector4f, ScalesData, [FResizeConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
		static ECoordTransMode CoordTransModeFromString(const TCHAR* StringVal);
		static EMode ModeFromString(const TCHAR* StringVal);
		static ENearestMode NearestModeFromString(const TCHAR* StringVal);
	};
} // UE::NNEHlslShaders::Internal
