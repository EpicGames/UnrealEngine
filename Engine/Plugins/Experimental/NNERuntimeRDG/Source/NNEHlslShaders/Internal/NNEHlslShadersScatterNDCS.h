// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{

    enum class EScatterNDReductionType : uint8
	{
		None = 0,
		Add,
		Mul,
		Max,
		Min,
		MAX
	};

	class FScatterNDConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 768 };
	};

	class NNEHLSLSHADERS_API FScatterNDCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FScatterNDCS);
		SHADER_USE_PARAMETER_STRUCT(FScatterNDCS, FHlslShaderBase)

		class FScatterNDNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FScatterNDConstants::MAX_NUM_DIMENSIONS);
		class FReduceType : SHADER_PERMUTATION_ENUM_CLASS("REDUCE_OPERATOR_TYPE", EScatterNDReductionType);
		using FPermutationDomain = TShaderPermutationDomain<FScatterNDNumDimensions, FReduceType>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, InputIndices)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputUpdates)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, DataTensorInfo, [FScatterNDConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FUintVector4, OutputTensorInfo, [FScatterNDConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(uint32, PartialIndexRank)
			SHADER_PARAMETER(uint32, SliceSize)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	
        static EScatterNDReductionType ReductionFromString(const TCHAR* StringVal);
    };
} // UE::NNEHlslShaders::Internal
