// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNE::Internal { class FTensor; }

namespace UE::NNEHlslShaders::Internal
{
	enum class ESoftmaxOperatorType : uint8
	{
		SOFTMAX,
		LOG_SOFTMAX,
		MAX
	};

	class FSoftmaxConstants
	{
	public:
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API TSoftmaxCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(TSoftmaxCS);
		SHADER_USE_PARAMETER_STRUCT(TSoftmaxCS, FHlslShaderBase)

		class FSoftmaxType : SHADER_PERMUTATION_ENUM_CLASS("SOFTMAX_OPERATOR_TYPE", ESoftmaxOperatorType);
		class FSingleDimension : SHADER_PERMUTATION_BOOL("SINGLE_DIMENSION");
		using FPermutationDomain = TShaderPermutationDomain<FSoftmaxType,FSingleDimension>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
			SHADER_PARAMETER(uint32, AxisSize)
			SHADER_PARAMETER(uint32, AfterAxisSize)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputSumExp)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal