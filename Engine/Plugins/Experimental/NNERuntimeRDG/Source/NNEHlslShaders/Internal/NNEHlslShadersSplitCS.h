// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{

	class FSplitConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 MAX_NUM_SPLITS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FSplitCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FSplitCS);
		SHADER_USE_PARAMETER_STRUCT(FSplitCS, FHlslShaderBase)

		class FSplitRank : SHADER_PERMUTATION_RANGE_INT("RANK", 1, FSplitConstants::MAX_NUM_DIMENSIONS);
		class FSplitAxis : SHADER_PERMUTATION_RANGE_INT("AXIS", 0, FSplitConstants::MAX_NUM_DIMENSIONS - 1);
		class FSplitNumSplits : SHADER_PERMUTATION_RANGE_INT("NUM_SPLITS", 1, FSplitConstants::MAX_NUM_SPLITS);
		using FPermutationDomain = TShaderPermutationDomain<FSplitRank, FSplitAxis, FSplitNumSplits>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV_ARRAY(RWBuffer<float>, Output, [FSplitConstants::MAX_NUM_SPLITS])
			SHADER_PARAMETER_ARRAY(FUintVector4, InputTensorInfo, [FSplitConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER_ARRAY(FUintVector4, OutputTensorInfo, [FSplitConstants::MAX_NUM_SPLITS * FSplitConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, ThreadCountX)
            SHADER_PARAMETER(uint32, Num)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
