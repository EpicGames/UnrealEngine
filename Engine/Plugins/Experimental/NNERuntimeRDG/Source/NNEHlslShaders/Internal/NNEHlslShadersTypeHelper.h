// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNETypes.h"
#include "PixelFormat.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class ENNEShaderDataType : uint8
	{
		BOOL,
		FLOAT16,
		FLOAT32,
		INT8,
		INT16,
		INT32,
		UINT8,
		UINT16,
		UINT32,

		MAX
	};

	const TCHAR	       NNEHLSLSHADERS_API *ShaderDataTypeToName(ENNEShaderDataType ShaderType);
	ENNEShaderDataType NNEHLSLSHADERS_API TensorToShaderDataType(ENNETensorDataType TensorType);
	EPixelFormat	   NNEHLSLSHADERS_API TensorDataTypeToPixelFormat(ENNETensorDataType TensorType);

} // UE::NNEHlslShaders::Internal
