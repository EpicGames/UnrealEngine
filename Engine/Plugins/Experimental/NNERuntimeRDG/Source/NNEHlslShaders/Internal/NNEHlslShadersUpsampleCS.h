// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNEHlslShadersBase.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"

namespace UE::NNEHlslShaders::Internal
{
	enum class EUpsampleMode : uint8
	{
		Nearest = 0,
		Bilinear,
		Trilinear,
		MAX
	};

	class FUpsampleConstants
	{
	public:

		static const int32 MAX_NUM_DIMENSIONS{ 8 };
		static const int32 NUM_GROUP_THREADS{ 256 };
	};

	class NNEHLSLSHADERS_API FUpsampleCS : public FHlslShaderBase
	{
		DECLARE_GLOBAL_SHADER(FUpsampleCS);
		SHADER_USE_PARAMETER_STRUCT(FUpsampleCS, FHlslShaderBase)

		class FUpsampleNumDimensions : SHADER_PERMUTATION_RANGE_INT("NUM_DIMENSIONS", 1, FUpsampleConstants::MAX_NUM_DIMENSIONS);
		class FUpsampleMode : SHADER_PERMUTATION_ENUM_CLASS("MODE", EUpsampleMode);
		using FPermutationDomain = TShaderPermutationDomain<FUpsampleNumDimensions, FUpsampleMode>;

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, Input)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, Output)
			SHADER_PARAMETER_ARRAY(FUintVector4, TensorInfo, [FUpsampleConstants::MAX_NUM_DIMENSIONS])
			SHADER_PARAMETER(uint32, Num)
			SHADER_PARAMETER(uint32, ThreadCountX)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};
} // UE::NNEHlslShaders::Internal
