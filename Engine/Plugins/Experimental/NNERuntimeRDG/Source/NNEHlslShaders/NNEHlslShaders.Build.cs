// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNEHlslShaders : ModuleRules
{
	public NNEHlslShaders( ReadOnlyTargetRules Target ) : base( Target )
	{
        PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"Projects",
				"RenderCore",
				"RHI",
				"NNE"
			}
		);
	}
}
