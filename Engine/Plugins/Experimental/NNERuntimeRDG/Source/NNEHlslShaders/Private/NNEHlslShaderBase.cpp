// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersBase.h"
#include "DataDrivenShaderPlatformInfo.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FHlslShaderBase::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsNNEShaders(Parameters.Platform);
	}
	
} // UE::NNEHlslShaders::Internal