// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersGatherElementsCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FGatherElementsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FGatherElementsConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FGatherElementsCS, "/NNEHlslShaders/NNEHlslShadersGatherElements.usf", "GatherElements", SF_Compute);
} // UE::NNEHlslShaders::Internal
