// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersBatchNormalizationCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void TBatchNormalizationCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FBatchNormalizationConstants::NUM_GROUP_THREADS);
	}

	IMPLEMENT_GLOBAL_SHADER(TBatchNormalizationCS, "/NNEHlslShaders/NNEHlslShadersBatchNormalization.usf", "BatchNormalization", SF_Compute);
} // UE::NNEHlslShaders::Internal
