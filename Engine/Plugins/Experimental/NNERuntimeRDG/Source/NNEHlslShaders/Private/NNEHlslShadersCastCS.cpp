// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersCastCS.h"

#include "NNE.h"
#include "NNEHlslShadersTypeHelper.h"
#include "ShaderCompilerCore.h"

namespace UE::NNEHlslShaders::Internal
{
	bool TCastCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		static TArray<ENNEShaderDataType> SupportedTypes =
		{
			ENNEShaderDataType::FLOAT16,
			ENNEShaderDataType::FLOAT32,
			ENNEShaderDataType::INT32
		};

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ENNEShaderDataType InputType = PermutationVector.Get<TCastCS::FInputType>();
		ENNEShaderDataType OutputType = PermutationVector.Get<TCastCS::FOutputType>();

		return SupportedTypes.Contains(InputType) && SupportedTypes.Contains(OutputType);
	}

	void TCastCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FCastConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
		ENNEShaderDataType InputType = PermutationVector.Get<TCastCS::FInputType>();
		ENNEShaderDataType OutputType = PermutationVector.Get<TCastCS::FOutputType>();
		OutEnvironment.SetDefine(TEXT("INPUT_TYPE"),  ShaderDataTypeToName(InputType));
		OutEnvironment.SetDefine(TEXT("OUTPUT_TYPE"), ShaderDataTypeToName(OutputType));

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowRealTypes);
	}

	IMPLEMENT_GLOBAL_SHADER(TCastCS, "/NNEHlslShaders/NNEHlslShadersCast.usf", "Cast", SF_Compute);
} // UE::NNEHlslShaders::Internal
