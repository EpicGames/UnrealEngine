// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvCS.h"
#include "NNE.h"
#include "RHIGlobals.h"

namespace UE::NNEHlslShaders::Internal
{
	namespace ConvUtils
	{

	TArray<int32> GetXBlockShape(TArrayView<const int32> GroupShape, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides)
	{
		check(WShape.Num() > 2);
		check(GroupShape.Num() == WShape.Num() - 2);
		check(Dilations.Num() == 0 || Dilations.Num() == GroupShape.Num());
		check(Strides.Num() == 0 || Strides.Num() == GroupShape.Num());
		
		TArray<int32> Result;
		Result.SetNumUninitialized(GroupShape.Num());
		for (int32 i = 0; i < GroupShape.Num(); i++)
		{
			int32 DilatedKernelSize = (i < Dilations.Num() ? Dilations[i] : 1) * (WShape[2 + i] - 1) + 1;
			Result[i] = DilatedKernelSize + (GroupShape[i] - 1) * (i < Strides.Num() ? Strides[i] : 1); 
		}
		return Result;
	}

	int32 GetNumThreadsPerGroup(EConvGroupSize GroupSize)
	{
		int32 NumThreadsPerGroup = 128;
		switch (GroupSize)
		{
			case EConvGroupSize::Size128:
				NumThreadsPerGroup = 128;
				break;
			case EConvGroupSize::Size256:
				NumThreadsPerGroup = 256;
				break;
			case EConvGroupSize::Size512:
				NumThreadsPerGroup = 512;
				break;
			default:
				check(false);
				break;
		}
		check(FMath::Log2((float)NumThreadsPerGroup) == FMath::Floor(FMath::Log2((float)NumThreadsPerGroup)));
		return NumThreadsPerGroup;
	}

	TArray<int32> GetGridShape(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape)
	{
		check(YShape.Num() > 2);
		check(YShape.Num() == (GroupShape.Num() + 2));

		TArray<int32> Result;
		Result.SetNumUninitialized(YShape.Num() - 2);
		for (int32 i = 2; i < YShape.Num(); i++)
		{
			Result[i-2] = FMath::DivideAndRoundUp(YShape[i], GroupShape[i-2]);
		}

		return Result;
	}

	uint32 GetTotalSharedMemoryUsed(uint32 ThreadGroupSize, uint32 Log2NumReadsPerThread)
	{
		return ThreadGroupSize * (1 + (1 << Log2NumReadsPerThread)) * sizeof(float);
	}

	} // namespace ConvUtils

	bool FConvCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(InParameters))
		{
			return false;
		}

		FPermutationDomain PermutationVector(InParameters.PermutationId);

		int Log2NumReadsPerThread = PermutationVector.Get<FConvNumReadsPerThread>();
		int32 ConvGroupSize = ConvUtils::GetNumThreadsPerGroup(PermutationVector.Get<FConvGroupSize>());

		if(ConvUtils::GetTotalSharedMemoryUsed(ConvGroupSize, Log2NumReadsPerThread) > GetMaxComputeSharedMemory())
		{
			return false;
		}
		
		return true;
	}

	void FConvCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_DIMENSIONS"), FConvConstants::MAX_NUM_DIMENSIONS);
	}

	// Returns the biggest thread group size that allows to keep in shared memory the portion of the input tensor and the filter necessary to compute the portion of output tended by it (the thread group).
	// If no suitable thread group size was found, EConvGroupSize::MAX is returned.
	EConvGroupSize FConvCS::GetBiggestCompatibleGroupSize(TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides)
	{
		check(WShape.Num() > 2);
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		
		for(EConvGroupSize TargetGroupSize = (EConvGroupSize)((uint8)EConvGroupSize::MAX - 1); (uint8)TargetGroupSize > 0; TargetGroupSize = (EConvGroupSize)((uint8)TargetGroupSize - 1))
		{
			const int32 NumDimensions = WShape.Num() - 2;
			const TArray<int32> GroupShape = GetGroupShape(TargetGroupSize, NumDimensions);
			TArray<int32> TargetXBlockShape = ConvUtils::GetXBlockShape(GroupShape, WShape, Dilations, Strides);
			int32 XWindowSize = 1;
			for (int32 Idx = 0; Idx < NumDimensions; Idx++)
			{
				XWindowSize *= TargetXBlockShape[Idx];
			}
			int32 NumReadsPerThread = FMath::DivideAndRoundUp(XWindowSize, ConvUtils::GetNumThreadsPerGroup(TargetGroupSize));
			int32 Log2NumReadsPerThread = FMath::Max(FMath::RoundToPositiveInfinity(FMath::Log2((float)NumReadsPerThread)), FConvConstants::MIN_NUM_READS_PER_THREAD_POW2);
			uint32 TotSharedMemoryUsed = ConvUtils::GetTotalSharedMemoryUsed(ConvUtils::GetNumThreadsPerGroup(TargetGroupSize), Log2NumReadsPerThread);

			if(TotSharedMemoryUsed <= GetMaxComputeSharedMemory() && Log2NumReadsPerThread <= FConvConstants::MAX_NUM_READS_PER_THREAD_POW2)
			{
				return TargetGroupSize;
			}
		}

		return EConvGroupSize::MAX;
	}

	TArray<int32> FConvCS::GetPadding(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads)
	{
		check(XShape.Num() > 2);
		check(WShape.Num() == XShape.Num());
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(AutoPad != EConvAutoPad::NOTSET || Pads.Num() == 2 * (WShape.Num() - 2));

		int32 NumDimensions = XShape.Num() - 2;
		TArray<int32> Result;
		Result.Init(0, 2 * NumDimensions);

		if (AutoPad == EConvAutoPad::NOTSET)
		{
			return TArray<int32>{Pads};
		}
		else if (AutoPad == EConvAutoPad::VALID)
		{
			return Result;
		}

		for (int32 DimensionIndex = 0; DimensionIndex < NumDimensions; DimensionIndex++)
		{
			int32 DilatedKernelSize = (DimensionIndex < Dilations.Num() ? Dilations[DimensionIndex] : 1) * (WShape[DimensionIndex + 2] - 1) + 1;
			int32 LastOutputIdx = ((int32) XShape[DimensionIndex + 2] + (DimensionIndex < Strides.Num() ? Strides[DimensionIndex] : 1) - 1) / (DimensionIndex < Strides.Num() ? Strides[DimensionIndex] : 1) - 1;
			int32 TotalPad = LastOutputIdx * (DimensionIndex < Strides.Num() ? Strides[DimensionIndex] : 1) + DilatedKernelSize - XShape[DimensionIndex + 2];
			TotalPad = TotalPad >= 0 ? TotalPad : 0;
			if (AutoPad == EConvAutoPad::SAME_LOWER)
			{
				Result[DimensionIndex] = (TotalPad + 1) / 2;
			}
			else
			{
				Result[DimensionIndex] = TotalPad / 2;
			}
			Result[NumDimensions + DimensionIndex] = TotalPad - Result[DimensionIndex];
		}

		return Result;
	}

	TArray<int32> FConvCS::GetOutputShape(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads)
	{
		check(XShape.Num() > 2);
		check(WShape.Num() == XShape.Num());
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(Pads.Num() == 0 || Pads.Num() == 2 * (WShape.Num() - 2));

		TArray<int32> Padding = GetPadding(XShape, WShape, AutoPad, Dilations, Strides, Pads);

		TArray<int32> Result;
		Result.SetNumUninitialized(XShape.Num());
		Result[0] = XShape[0];
		Result[1] = WShape[0];

		int32 NumDimensions = XShape.Num() - 2;
		for (int32 DimensionIndex = 0; DimensionIndex < NumDimensions; DimensionIndex++)
		{
			int32 PaddedSize = XShape[2 + DimensionIndex] + Padding[DimensionIndex] + Padding[NumDimensions + DimensionIndex];
			int32 DilatedKernelSize = (DimensionIndex < Dilations.Num() ? Dilations[DimensionIndex] : 1) * (WShape[2 + DimensionIndex] - 1) + 1;
			float PaddeSizeMinusDilatedKernelSize = (float)(PaddedSize - DilatedKernelSize);
			int32 OutputSize = (int32)(PaddeSizeMinusDilatedKernelSize / (DimensionIndex < Strides.Num() ? (float)Strides[DimensionIndex] : 1.0) + 1);
			Result[2 + DimensionIndex] = OutputSize;
		}

		return Result;
	}

	void FConvCS::FillInParameters(EConvGroupSize GroupSize, TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, bool HasB,
			EConvAutoPad AutoPad, int Group, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, FConvCS::FParameters& Parameters)
	{
		check(XShape.Num() > 2);
		check(WShape.Num() == XShape.Num());
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(Pads.Num() == 0 || Pads.Num() == 2 * (WShape.Num() - 2));
		check(GroupSize != EConvGroupSize::MAX);
		check(WShape[0] > 0)
		check(WShape[1] > 0)

		int32 NumDimensions = XShape.Num() - 2;

		TArray<int32> Padding = GetPadding(XShape, WShape, AutoPad, Dilations, Strides, Pads);
		TArray<int32> GroupShape = GetGroupShape(GroupSize, NumDimensions);
		TArray<int32> YShape = GetOutputShape(XShape, WShape, AutoPad, Dilations, Strides, Pads);
		TArray<int32> GridShape = ConvUtils::GetGridShape(YShape, GroupShape);
		TArray<int32> XBlockShape = ConvUtils::GetXBlockShape(GroupShape, WShape, Dilations, Strides);
		
		int32 GroupStride = 1;
		int32 GroupThreadStride = 1;
		int32 XBlockSize = 1;
		int32 YMemoryStride = 1;
		int32 XMemoryStride = 1;
		int32 WChannelSize = 1;
		for (int32 i = NumDimensions-1; i >= 0; i--)
		{
			int32 Stride = i < Strides.Num() ? Strides[i] : 1;
			int32 Dilation = i < Dilations.Num() ? Dilations[i] : 1;
			Parameters.Dilation_Stride_XBlockStartOffset_DilationXBlockStride[i] = FIntVector4(Dilation, Stride, -Padding[i], Dilation * XBlockSize);
			Parameters.GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[i] = FIntVector4(GroupStride, GroupShape[i], GroupThreadStride, Stride * XBlockSize);
			Parameters.YDimension_YMemoryStride_XDimension_XMemoryStride[i] = FIntVector4(YShape[2 + i], YMemoryStride, XShape[2 + i], XMemoryStride);
			Parameters.XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[i] = FIntVector4(GroupShape[i] * Stride, XBlockSize, WShape[2 + i], WShape[2 + i] * Dilation * XBlockSize);
			Parameters.OneDiv_GroupStride_GroupThreadStride_XBlockStride[i] = FVector4f(1.0/((float)GroupStride), 1.0/((float)GroupThreadStride), 1.0/((float)XBlockSize), 0.0);

			GroupStride *= GridShape[i];
			GroupThreadStride *= GroupShape[i];
			XBlockSize *= XBlockShape[i];
			YMemoryStride *= YShape[2 + i];
			XMemoryStride *= XShape[2 + i];
			WChannelSize *= WShape[2 + i];
		}
		
		Parameters.NumWFeatures = WShape[0];
		Parameters.NumWChannels = WShape[1];

		Parameters.YBatchStride = YShape[1] * YMemoryStride;
		Parameters.YOutputKernelStride = YMemoryStride;
		
		Parameters.XBatchStride = XShape[1] * XMemoryStride;
		Parameters.XChannelStride = XMemoryStride;

		Parameters.XBlockSize = XBlockSize;

		Parameters.NumChannelsPerBatch = FMath::Min((int32)((float)GroupThreadStride / (float)WChannelSize), (int32)WShape[1]);
		check(Parameters.NumChannelsPerBatch > 0)
		Parameters.NumChannelBatches = FMath::DivideAndRoundUp((int32)WShape[1], Parameters.NumChannelsPerBatch);
		
		Parameters.WOutputKernelStride = WShape[1] * WChannelSize;
		Parameters.WChannelBatchSize = Parameters.NumChannelsPerBatch * WChannelSize;
		Parameters.WChannelSize = WChannelSize;

		Parameters.GroupsDivM = (float)Group / (float)WShape[0];
	}

	int32 FConvCS::GetNumReadsPerThread(EConvGroupSize GroupSize, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides)
	{
		check(WShape.Num() > 2);
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);

		int32 NumDimensions = WShape.Num() - 2;

		TArray<int32> GroupShape = GetGroupShape(GroupSize, NumDimensions);
		int32 NumThreadsPerGroup = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			NumThreadsPerGroup *= GroupShape[i];
		}

		TArray<int32> XBlockShape = ConvUtils::GetXBlockShape(GroupShape, WShape, Dilations, Strides);
		int32 NumXBlockElements = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			NumXBlockElements *= XBlockShape[i];
		}

		int32 NumReads = FMath::DivideAndRoundUp(NumXBlockElements, NumThreadsPerGroup);
		int32 NumReadsPow2 = FMath::Max(FMath::RoundToPositiveInfinity(FMath::Log2((float)NumReads)), FConvConstants::MIN_NUM_READS_PER_THREAD_POW2);

		if(NumReadsPow2 <= FConvConstants::MAX_NUM_READS_PER_THREAD_POW2)
		{
			return NumReadsPow2;
		}

		return -1;
	}

	TArray<int32> FConvCS::GetGroupShape(EConvGroupSize GroupSize, int32 NumDimensions)
	{
		check(NumDimensions > 0);

		int32 NumThreadsPerGroup = ConvUtils::GetNumThreadsPerGroup(GroupSize);

		int32 Power = (int32)FMath::Log2((float)NumThreadsPerGroup);
		int32 MinPowerPerDim = (int32)((float)Power / (float)NumDimensions);
		int32 PowerReminder = Power - NumDimensions * MinPowerPerDim;

		TArray<int32> Result;
		Result.Init((int32)FMath::Pow((float)2.0, (float)MinPowerPerDim), NumDimensions);
		for (int32 i = 0; i < PowerReminder; i++)
		{
			Result[NumDimensions-1-i] *= 2;
		}
		return Result;
	}

	FIntVector FConvCS::GetGroupCount(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape)
	{
		check(YShape.Num() > 2);
		check(YShape.Num() == (GroupShape.Num() + 2));

		int32 ThreadGroupCountValueX = 1;
		for (int32 i = 2; i < YShape.Num(); i++)
		{
			ThreadGroupCountValueX *= FMath::DivideAndRoundUp(YShape[i], GroupShape[i-2]);
		}

		return FIntVector(ThreadGroupCountValueX, YShape[1], YShape[0]);
	}

	EConvGroupSize FConvCS::GetMinimalGroupSize(TArrayView<const int32> WShape)
	{
		const int32 NumDimensions = WShape.Num() - 2;
		int32 WChannelSize = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			WChannelSize *= WShape[2 + i];
		}
		
		for (int32 i = 0; i < (int32)EConvGroupSize::MAX; i++)
		{
			if (ConvUtils::GetNumThreadsPerGroup((EConvGroupSize)i) >= WChannelSize)
			{
				return (EConvGroupSize)i;
			}
		}
		
		return EConvGroupSize::MAX;
	}

	void FConvCS::LexFromString(EConvAutoPad& OutValue, const TCHAR* StringVal)
	{
		OutValue = EConvAutoPad::NOTSET;
		if (FCString::Stricmp(StringVal, TEXT("NOTSET")) == 0) OutValue = EConvAutoPad::NOTSET;
		else if (FCString::Stricmp(StringVal, TEXT("SAME_UPPER")) == 0) OutValue = EConvAutoPad::SAME_UPPER;
		else if (FCString::Stricmp(StringVal, TEXT("SAME_LOWER")) == 0) OutValue = EConvAutoPad::SAME_LOWER;
		else if (FCString::Stricmp(StringVal, TEXT("VALID")) == 0) OutValue = EConvAutoPad::VALID;
	}

	IMPLEMENT_GLOBAL_SHADER(FConvCS, "/NNEHlslShaders/NNEHlslShadersConv.usf", "Conv", SF_Compute);
} // UE::NNEHlslShaders::Internal