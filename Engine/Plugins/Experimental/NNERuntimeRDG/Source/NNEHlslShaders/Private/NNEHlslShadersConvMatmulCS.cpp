// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvMatmulCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	FIntVector FConvMatmulCS::GetGroupCount(TConstArrayView<uint32> OutputShape)
	{
		constexpr int32 NUM_PIXEL_PER_THREADGROUP = 32 ;
		constexpr int32 NUM_CHANNEL_PER_THREADGROUP = 32 ;

		check(OutputShape.Num() == 4);

		int32 NumBatch = OutputShape[0];
		int32 NumChannel = OutputShape[1];
		int32 NumPixel = OutputShape[2] * OutputShape[3];

		return FIntVector(
			FMath::DivideAndRoundUp(NumPixel, NUM_PIXEL_PER_THREADGROUP),
			FMath::DivideAndRoundUp(NumChannel, NUM_CHANNEL_PER_THREADGROUP),
			NumBatch);
	}

	IMPLEMENT_GLOBAL_SHADER(FConvMatmulCS, "/NNEHlslShaders/NNEHlslShadersConvMatmul.usf", "ConvMatmul", SF_Compute);
} // UE::NNEHlslShaders::Internal