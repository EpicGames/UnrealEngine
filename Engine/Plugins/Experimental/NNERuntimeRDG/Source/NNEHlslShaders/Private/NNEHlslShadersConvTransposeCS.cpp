// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvTransposeCS.h"

namespace UE::NNEHlslShaders::Internal
{
	void FConvTransposeCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_STACK_DIMENSIONS"), FConvTransposeConstants::MAX_NUM_DIMENSIONS);
	}

	TArray<int32> FConvTransposeCS::GetOutputShape(TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, EConvTransposeAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding, int32 Group)
	{
		check(XShape.Num() > 2);
		check(WShape.Num() == XShape.Num());
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(AutoPad != EConvTransposeAutoPad::NOTSET || Pads.Num() == 2 * (WShape.Num() - 2));

		TArray<int32> Padding = GetPadding(WShape, AutoPad, Dilations, Strides, Pads, OutputPadding);

		TArray<int32> Result;
		Result.SetNumUninitialized(XShape.Num());
		Result[0] = XShape[0];
		Result[1] = WShape[1] * Group;

		int32 NumDimensions = XShape.Num() - 2;
		for (int32 DimensionIndex = 0; DimensionIndex < NumDimensions; DimensionIndex++)
		{
			Result[DimensionIndex + 2] = Strides[DimensionIndex] * (XShape[DimensionIndex + 2] - 1) + OutputPadding[DimensionIndex] +
				((WShape[DimensionIndex + 2] - 1) * Dilations[DimensionIndex] + 1) -
				Padding[DimensionIndex] - Padding[Strides.Num() + DimensionIndex];
		}

		return Result;
	}

	void FConvTransposeCS::FillInParameters(EConvTransposeGroupSize GroupSize, TArrayView<const uint32> XShape, TArrayView<const uint32> WShape, bool HasB, EConvTransposeAutoPad AutoPad, int32 Group, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding, FConvTransposeCS::FParameters& Parameters)
	{
		check(XShape.Num() > 2);
		check(WShape.Num() == XShape.Num());
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(AutoPad != EConvTransposeAutoPad::NOTSET || Pads.Num() == 2 * (WShape.Num() - 2));
		check(GetNumReadsPerThread(GroupSize, WShape, Dilations, Strides) >= 0)
			check(WShape[0] > 0)
			check(WShape[1] > 0)

			int32 NumDimensions = XShape.Num() - 2;

		TArray<int32> Padding = GetPadding(WShape, AutoPad, Dilations, Strides, Pads, OutputPadding);
		TArray<int32> GroupShape = GetGroupShape(GroupSize, NumDimensions);
		TArray<int32> YShape = GetOutputShape(XShape, WShape, AutoPad, Dilations, Strides, Pads, OutputPadding, Group);
		TArray<int32> GridShape = GetGridShape(YShape, GroupShape);
		TArray<int32> XBlockShape = GetXBlockShape(GroupShape, WShape, Dilations, Strides);

		int32 GroupStride = 1;
		int32 GroupThreadStride = 1;
		int32 XBlockSize = 1;
		int32 YMemoryStride = 1;
		int32 XMemoryStride = 1;
		int32 WChannelSize = 1;
		for (int32 i = NumDimensions - 1; i >= 0; i--)
		{
			int32 Stride = i < Strides.Num() ? Strides[i] : 1;
			int32 Dilation = i < Dilations.Num() ? Dilations[i] : 1;
			Parameters.Dilation_Stride_XBlockStartOffset_DilationXBlockStride[i] = FIntVector4(Dilation, Stride, Padding[i] + Dilations[i] * (1 - WShape[2 + i]), Dilation * XBlockSize);
			Parameters.GroupStride_GroupShape_GroupThreadStride_StrideXBlockStride[i] = FIntVector4(GroupStride, GroupShape[i], GroupThreadStride, XBlockSize);
			Parameters.YDimension_YMemoryStride_XDimension_XMemoryStride[i] = FIntVector4(YShape[2 + i], YMemoryStride, XShape[2 + i], XMemoryStride);
			Parameters.XBlockStartStride_XBlockStride_WDimension_WDimensionDilationXBlockStride[i] = FIntVector4(GroupShape[i], XBlockSize, WShape[2 + i], WShape[2 + i] * Dilation * XBlockSize);
			Parameters.OneDiv_GroupStride_GroupThreadStride_OneDivStride[i] = FVector4f(1.0 / ((float)GroupStride), 1.0 / ((float)GroupThreadStride), 1.0 / ((float)XBlockSize), 1.0 / ((float)Stride));

			GroupStride *= GridShape[i];
			GroupThreadStride *= GroupShape[i];
			XBlockSize *= XBlockShape[i];
			YMemoryStride *= YShape[2 + i];
			XMemoryStride *= XShape[2 + i];
			WChannelSize *= WShape[2 + i];
		}

		Parameters.NumWChannels = WShape[0];
		Parameters.NumOutChannelsDivGroup = WShape[1];

		Parameters.YBatchStride = YShape[1] * YMemoryStride;
		Parameters.YOutputKernelStride = YMemoryStride;

		Parameters.XBatchStride = XShape[1] * XMemoryStride;
		Parameters.XChannelStride = XMemoryStride;

		Parameters.XBlockSize = XBlockSize;

		Parameters.NumChannelsPerBatch = FMath::Min((int32)((float)GroupThreadStride / (float)WChannelSize), (int32)WShape[0]);
		check(Parameters.NumChannelsPerBatch > 0)
			Parameters.NumChannelBatches = FMath::DivideAndRoundUp((int32)WShape[0], Parameters.NumChannelsPerBatch);

		Parameters.WOutputKernelStride = WShape[1] * WChannelSize;
		Parameters.WChannelBatchSize = Parameters.NumChannelsPerBatch * WShape[1] * WChannelSize;
		Parameters.WChannelSize = WChannelSize;

		Parameters.GroupsDivM = 1.0 / ((float)WShape[1] * Group);
		Parameters.OneDivGroup = 1.0 / (float)Group;
	}

	int32 FConvTransposeCS::GetNumReadsPerThread(EConvTransposeGroupSize GroupSize, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides)
	{
		check(WShape.Num() > 2);
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);

		int32 NumDimensions = WShape.Num() - 2;

		TArray<int32> GroupShape = GetGroupShape(GroupSize, NumDimensions);
		int32 NumThreadsPerGroup = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			NumThreadsPerGroup *= GroupShape[i];
		}

		TArray<int32> XBlockShape = GetXBlockShape(GroupShape, WShape, Dilations, Strides);
		int32 NumXBlockElements = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			NumXBlockElements *= XBlockShape[i];
		}

		int32 NumReads = FMath::DivideAndRoundUp(NumXBlockElements, NumThreadsPerGroup);
		int32 NumReadsPow2 = FMath::Max(FMath::RoundToPositiveInfinity(FMath::Log2((float)NumReads)), FConvTransposeConstants::MIN_NUM_READS_PER_THREAD_POW2);

		if (NumReadsPow2 <= FConvTransposeConstants::MAX_NUM_READS_PER_THREAD_POW2)
		{
			return NumReadsPow2;
		}

		return -1;
	}

	TArray<int32> FConvTransposeCS::GetGroupShape(EConvTransposeGroupSize GroupSize, int32 NumDimensions)
	{
		check(NumDimensions > 0);

		int32 NumThreadsPerGroup = GetNumThreadsPerGroup(GroupSize);

		int32 Power = (int32)FMath::Log2((float)NumThreadsPerGroup);
		int32 MinPowerPerDim = (int32)((float)Power / (float)NumDimensions);
		int32 PowerReminder = Power - NumDimensions * MinPowerPerDim;

		TArray<int32> Result;
		Result.Init((int32)FMath::Pow((float)2.0, (float)MinPowerPerDim), NumDimensions);
		for (int32 i = 0; i < PowerReminder; i++)
		{
			Result[NumDimensions - 1 - i] *= 2;
		}

		return Result;
	}

	FIntVector FConvTransposeCS::GetGroupCount(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape)
	{
		check(YShape.Num() > 2);
		check(YShape.Num() == (GroupShape.Num() + 2));

		int32 ThreadGroupCountValueX = 1;
		for (int32 i = 2; i < YShape.Num(); i++)
		{
			ThreadGroupCountValueX *= FMath::DivideAndRoundUp(YShape[i], GroupShape[i - 2]);
		}

		return FIntVector(ThreadGroupCountValueX, YShape[1], YShape[0]);
	}

	EConvTransposeGroupSize FConvTransposeCS::GetMinimalGroupSize(TArrayView<const uint32> WShape)
	{
		int32 NumDimensions = WShape.Num() - 2;
		int32 WChannelSize = 1;
		for (int32 i = 0; i < NumDimensions; i++)
		{
			WChannelSize *= WShape[2 + i];
		}

		for (int32 i = 0; i < (int32)EConvTransposeGroupSize::MAX; i++)
		{
			if (GetNumThreadsPerGroup((EConvTransposeGroupSize)i) >= WChannelSize)
			{
				return (EConvTransposeGroupSize)i;
			}
		}

		return EConvTransposeGroupSize::MAX;
	}

	TArray<int32> FConvTransposeCS::GetXBlockShape(TArrayView<const int32> GroupShape, TArrayView<const uint32> WShape, TArrayView<const int32> Dilations, TArrayView<const int32> Strides)
	{
		check(WShape.Num() > 2);
		check(GroupShape.Num() == WShape.Num() - 2);
		check(Dilations.Num() == 0 || Dilations.Num() == GroupShape.Num());
		check(Strides.Num() == 0 || Strides.Num() == GroupShape.Num());

		TArray<int32> Result;
		Result.SetNumUninitialized(GroupShape.Num());
		for (int32 i = 0; i < GroupShape.Num(); i++)
		{
			int32 DilatedKernelSize = (i < Dilations.Num() ? Dilations[i] : 1) * (WShape[2 + i] - 1) + 1;
			Result[i] = DilatedKernelSize + (GroupShape[i] - 1);
		}
		return Result;
	}

	TArray<int32> FConvTransposeCS::GetPadding(TArrayView<const uint32> WShape, EConvTransposeAutoPad AutoPad, TArrayView<const int32> Dilations, TArrayView<const int32> Strides, TArrayView<const int32> Pads, TArrayView<const int32> OutputPadding)
	{
		check(WShape.Num() > 2);
		check(Dilations.Num() == 0 || Dilations.Num() == WShape.Num() - 2);
		check(Strides.Num() == 0 || Strides.Num() == WShape.Num() - 2);
		check(AutoPad != EConvTransposeAutoPad::NOTSET || Pads.Num() == 2 * (WShape.Num() - 2));

		int32 NumDimensions = WShape.Num() - 2;
		TArray<int32> Result;
		Result.Init(0, 2 * NumDimensions);

		if (AutoPad == EConvTransposeAutoPad::NOTSET)
		{
			return TArray<int32>{Pads};
		}
		else if (AutoPad == EConvTransposeAutoPad::VALID)
		{
			return Result;
		}

		for (int32 DimensionIndex = 0; DimensionIndex < NumDimensions; DimensionIndex++)
		{
			int32 TotalPad = (WShape[DimensionIndex + 2] - 1) * Dilations[DimensionIndex] - Strides[DimensionIndex] + OutputPadding[DimensionIndex] + 1;

			if (AutoPad == EConvTransposeAutoPad::SAME_LOWER)
			{
				Result[DimensionIndex] = (TotalPad + 1) / 2;
			}
			else
			{
				Result[DimensionIndex] = TotalPad / 2;
			}
			Result[NumDimensions + DimensionIndex] = TotalPad - Result[DimensionIndex];
		}

		return Result;
	}

	int32 FConvTransposeCS::GetNumThreadsPerGroup(EConvTransposeGroupSize GroupSize)
	{
		int32 NumThreadsPerGroup = 128;
		switch (GroupSize)
		{
		case EConvTransposeGroupSize::Size128:
			NumThreadsPerGroup = 128;
			break;
		case EConvTransposeGroupSize::Size256:
			NumThreadsPerGroup = 256;
			break;
		case EConvTransposeGroupSize::Size512:
			NumThreadsPerGroup = 512;
			break;
		default:
			NumThreadsPerGroup = 128;
			break;
		}
		check(FMath::Log2((float)NumThreadsPerGroup) == FMath::Floor(FMath::Log2((float)NumThreadsPerGroup)));
		return NumThreadsPerGroup;
	}

	TArray<int32> FConvTransposeCS::GetGridShape(TArrayView<const int32> YShape, TArrayView<const int32> GroupShape)
	{
		check(YShape.Num() > 2);
		check(YShape.Num() == (GroupShape.Num() + 2));

		TArray<int32> Result;
		Result.SetNumUninitialized(YShape.Num() - 2);
		for (int32 i = 2; i < YShape.Num(); i++)
		{
			Result[i - 2] = FMath::DivideAndRoundUp(YShape[i], GroupShape[i - 2]);
		}

		return Result;
	}

	void FConvTransposeCS::LexFromString(EConvTransposeAutoPad& OutValue, const TCHAR* StringVal)
	{
		OutValue = EConvTransposeAutoPad::NOTSET;
		if (FCString::Stricmp(StringVal, TEXT("NOTSET")) == 0) OutValue = EConvTransposeAutoPad::NOTSET;
		else if (FCString::Stricmp(StringVal, TEXT("SAME_UPPER")) == 0) OutValue = EConvTransposeAutoPad::SAME_UPPER;
		else if (FCString::Stricmp(StringVal, TEXT("SAME_LOWER")) == 0) OutValue = EConvTransposeAutoPad::SAME_LOWER;
		else if (FCString::Stricmp(StringVal, TEXT("VALID")) == 0) OutValue = EConvTransposeAutoPad::VALID;
	}

	IMPLEMENT_GLOBAL_SHADER(FConvTransposeCS, "/NNEHlslShaders/NNEHlslShadersConvTranspose.usf", "ConvTranspose", SF_Compute);
} // UE::NNEHlslShaders::Internal