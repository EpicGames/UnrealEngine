// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvWinogradInputCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FConvWinogradInputCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FConvWinogradInputConstants::THREADGROUP_SIZE_X);
	}

	IMPLEMENT_GLOBAL_SHADER(FConvWinogradInputCS, "/NNEHlslShaders/NNEHlslShadersConvWinogradInput.usf", "ConvWinogradInput", SF_Compute);
} // UE::NNEHlslShaders::Internal