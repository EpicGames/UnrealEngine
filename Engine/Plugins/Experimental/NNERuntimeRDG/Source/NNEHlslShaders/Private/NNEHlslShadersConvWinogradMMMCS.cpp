// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvWinogradMMMCS.h"

#include <limits>
#include "NNEHlslShadersTypeHelper.h"
#include "NNE.h"
#include "ShaderCompilerCore.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FConvWinogradMMMCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
#if PLATFORM_MAC
		return false;
#else
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ENNEShaderDataType DataType = PermutationVector.Get<FConvWinogradMMMCS::FDataType>();
		return DataType == ENNEShaderDataType::FLOAT16 || DataType == ENNEShaderDataType::FLOAT32;
#endif // PLATFORM_MAC
	}

	void FConvWinogradMMMCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(InParameters.PermutationId);
		ENNEShaderDataType DataType = PermutationVector.Get<FConvWinogradMMMCS::FDataType>();
		OutEnvironment.SetDefine(TEXT("WORK_TYPE"), ShaderDataTypeToName(DataType));

		// Winograd does not compile on D3D12 SM6 + bindless if using vector<x,y> atm
		// hence the define here directly set to the vectorized type rather as a define of a templated vector in the shader
		if (DataType == ENNEShaderDataType::FLOAT16)
		{
			OutEnvironment.SetDefine(TEXT("WORK_TYPE_VECTOR"), TEXT("float16_t2"));
		}
		else if (DataType == ENNEShaderDataType::FLOAT32)
		{
			OutEnvironment.SetDefine(TEXT("WORK_TYPE_VECTOR"), TEXT("float2"));
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("WORK_TYPE_VECTOR"), TEXT("UNSUPPORTED_DATA_TYPE"));
		}

		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowRealTypes);
	}

	// Simple Heuristic that tries to find the optimal block size
	int FConvWinogradMMMCS::GetOptimalBlockSizeN(int M, int K, int N)
	{
		struct PerformanceData
		{
			int BlockSizeN;
			float LowFlops;
			float HighFlops;
		};

		// These numbers are based on performance measurements
		PerformanceData Performance[3] =
		{
			{ 16, 125, 250 },
			{ 32, 200, 280 },
			{ 64, 290, 300 }
		};

		bool UseHighFlops = M / N < 250;
		float SmallestDuration = std::numeric_limits<float>::infinity();
		int Result = 0;
		for (int i = 0; i < sizeof(Performance) / sizeof(Performance[0]); i++)
		{
			float Flops = UseHighFlops ? Performance[i].HighFlops : Performance[i].LowFlops;
			int Mask = Performance[i].BlockSizeN - 1;
			// Round up to the next multiple of BlocksSizeN
			int NExtended = (N + Mask) & ~Mask;
			float Duration = NExtended / Flops;
			if (SmallestDuration > Duration)
			{
				SmallestDuration = Duration;
				Result = Performance[i].BlockSizeN;
			}
		}
		return Result;
	}

	IMPLEMENT_GLOBAL_SHADER(FConvWinogradMMMCS, "/NNEHlslShaders/NNEHlslShadersConvWinogradMMM.usf", "ConvWinogradMMM", SF_Compute);
} // UE::NNEHlslShaders::Internal
