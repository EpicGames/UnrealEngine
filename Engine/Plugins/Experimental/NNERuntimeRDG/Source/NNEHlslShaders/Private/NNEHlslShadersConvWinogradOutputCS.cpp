// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvWinogradOutputCS.h"

#include "NNEHlslShadersTypeHelper.h"
#include "NNE.h"
#include "ShaderCompilerCore.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FConvWinogradOutputCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ENNEShaderDataType DataType = PermutationVector.Get<FConvWinogradOutputCS::FDataType>();
		return DataType == ENNEShaderDataType::FLOAT16 || DataType == ENNEShaderDataType::FLOAT32;
	}

	void FConvWinogradOutputCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(InParameters.PermutationId);
		ENNEShaderDataType DataType = PermutationVector.Get<FConvWinogradOutputCS::FDataType>();
		OutEnvironment.SetDefine(TEXT("WORK_TYPE"), ShaderDataTypeToName(DataType));

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FConvWinogradOutputConstants::THREADGROUP_SIZE_X);
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowRealTypes);
	}

	IMPLEMENT_GLOBAL_SHADER(FConvWinogradOutputCS, "/NNEHlslShaders/NNEHlslShadersConvWinogradOutput.usf", "ConvWinogradOutput", SF_Compute);
} // UE::NNEHlslShaders::Internal
