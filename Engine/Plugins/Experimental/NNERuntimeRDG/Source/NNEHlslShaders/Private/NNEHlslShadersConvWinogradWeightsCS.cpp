// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersConvWinogradWeightsCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FConvWinogradWeightsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FConvWinogradWeightsConstants::THREADGROUP_SIZE_X);
	}

	IMPLEMENT_GLOBAL_SHADER(FConvWinogradWeightsCS, "/NNEHlslShaders/NNEHlslShadersConvWinogradWeights.usf", "ConvWinogradWeights", SF_Compute);
} // UE::NNEHlslShaders::Internal