// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersCumSumCS.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "NNE.h"
#include "RenderGraphBuilder.h"

namespace UE::NNEHlslShaders::Internal
{
	void TInitCumSumCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INIT_THREADGROUP_SIZE"), FCumSumConstants::INIT_THREADGROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("INIT_SHADER"), 1);
	}

	void TCumSumCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), FCumSumConstants::THREADGROUP_SIZE);
		OutEnvironment.SetDefine(TEXT("VALUES_PER_THREAD"), FCumSumConstants::VALUES_PER_THREAD);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(TInitCumSumCS, "/NNEHlslShaders/NNEHlslShadersCumSum.usf", "InitCumSum", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(TCumSumCS, "/NNEHlslShaders/NNEHlslShadersCumSum.usf", "CumSum", SF_Compute);
} // UE::NNEHlslShaders::Internal
