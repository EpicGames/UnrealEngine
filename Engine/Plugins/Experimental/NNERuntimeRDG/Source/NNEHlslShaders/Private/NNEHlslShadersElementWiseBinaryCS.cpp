// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersElementWiseBinaryCS.h"
#include "NNEHlslShadersLog.h"

namespace UE::NNEHlslShaders::Internal
{
	void TElementWiseBinaryCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FElementWiseBinaryConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);

		const FString OpFunc = GetOpFunc(PermutationVector.Get<FOperatorType>());

		OutEnvironment.SetDefine(TEXT("ELEMENTWISE_OP(X,Y)"), *OpFunc);
	}

	const FString TElementWiseBinaryCS::GetOpFunc(EElementWiseBinaryOperatorType OpType)
	{
		FString OpTable[(int32) EElementWiseBinaryOperatorType::MAX];

		for (int32 Idx = 0; Idx < (int32) EElementWiseBinaryOperatorType::MAX; ++Idx)
		{
			OpTable[Idx] = FString("");
		}

#define OP(OpName, OpFunc) OpTable[(int32) EElementWiseBinaryOperatorType::OpName] = OpFunc
		OP(Add,              TEXT("((X)+(Y))"));
		//OP(And,            TEXT("((X)&&(Y))"));
		OP(Div,              TEXT("((X)/(Y))"));
		//OP(Equal,          TEXT("((X)==(Y))"));
		//OP(Greater,        TEXT("((X)>(Y))"));
		//OP(GreaterOrEqual, TEXT("((X)>=(Y))"));
		//OP(Less,           TEXT("((X)<(Y))"));
		//OP(LessOrEqual,    TEXT("((X)<(Y))"));
		OP(Mod,              TEXT("((X)%(Y))"));
		OP(Mul,              TEXT("((X)*(Y))"));
		//OP(Or,             TEXT("((X)||(Y))"));
		OP(Prelu,            TEXT("prelu(X,Y)"));
		OP(Pow,              TEXT("safe_pow(X,Y)"));
		OP(Sub,              TEXT("((X)-(Y))"));
		//OP(Or,             TEXT("((X)^=(Y))"));
#undef OP

		FString OpFunc = OpTable[(int32) OpType];

		if (OpFunc == "")
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Undefined ElementWise Binary operator name for operator:%d", int(OpType));
		}

		return OpFunc;
	}

	IMPLEMENT_GLOBAL_SHADER(TElementWiseBinaryCS, "/NNEHlslShaders/NNEHlslShadersElementWiseBinary.usf", "ElementWiseBinary", SF_Compute);
} // UE::NNEHlslShaders::Internal
