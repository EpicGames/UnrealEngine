// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersElementWiseUnaryCS.h"
#include "NNEHlslShadersLog.h"

namespace UE::NNEHlslShaders::Internal
{
	bool TElementWiseUnaryCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		EElementWiseUnaryOperatorType OpType = PermutationVector.Get<TElementWiseUnaryCS::FOperatorType>();
		const bool bAlphaOnGpu = PermutationVector.Get<TElementWiseUnaryCS::FAlphaOnGPU>();
		const bool bBetaOnGpu = PermutationVector.Get<TElementWiseUnaryCS::FBetaOnGPU>();

		return OpType == EElementWiseUnaryOperatorType::Clip || (!bAlphaOnGpu && !bBetaOnGpu);
	}

	void TElementWiseUnaryCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FElementWiseUnaryConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);

		const FString OpFunc = GetOpFunc(PermutationVector.Get<FOperatorType>());

		OutEnvironment.SetDefine(TEXT("ELEMENTWISE_OP(X)"), *OpFunc);
	}

	const FString TElementWiseUnaryCS::GetOpFunc(EElementWiseUnaryOperatorType OpType)
	{
		FString OpTable[(int32) EElementWiseUnaryOperatorType::MAX];

		for (int32 Idx = 0; Idx < (int32) EElementWiseUnaryOperatorType::MAX; ++Idx)
		{
			OpTable[Idx] = FString("");
		}

#define OP(OpName, OpFunc) OpTable[(int32) EElementWiseUnaryOperatorType::OpName] = OpFunc
		OP(Abs,         TEXT("abs(X)"));
		OP(Acos,        TEXT("acos(X)"));
		OP(Acosh,       TEXT("acosh(X)"));
		OP(Asin,        TEXT("asin(X)"));
		OP(Asinh,       TEXT("asinh(X)"));
		OP(Atan,        TEXT("atan(X)"));
		OP(Atanh,       TEXT("atanh(X)"));
		//OP(BitShift,  TEXT("bitshift(X)"));
		//OP(Cast,      TEXT("cast(X)"));
		OP(Ceil,        TEXT("ceil(X)"));
		OP(Clip,        TEXT("clipOp(X)"));
		OP(Cos,         TEXT("cos(X)"));
		OP(Cosh,        TEXT("cosh(X)"));
		OP(Elu,         TEXT("elu(X)"));
		OP(Erf,         TEXT("erf(X)"));
		OP(Exp,         TEXT("exp(X)"));
		OP(Floor,       TEXT("floor(X)"));
		OP(IsInf,       TEXT("isinf(X)"));
		OP(IsNan,       TEXT("isnan(X)"));//Note: There is a warning on PC FXC about input that can neither be Nan.
		OP(HardSigmoid, TEXT("hardSigmoid(X)"));
		OP(HardSwish,   TEXT("hardSwish(X)"));
		OP(LeakyRelu,   TEXT("leakyRelu(X)"));
		OP(Log,         TEXT("log(X)"));
		OP(Neg,         TEXT("-(X)"));
		//OP(Not,       TEXT("not(X)"));
		OP(Reciprocal,  TEXT("1 / (X)"));
		OP(Relu,        TEXT("relu(X)"));
		OP(Round,       TEXT("round(X)"));
		OP(Selu,        TEXT("selu(X)"));
		OP(Sigmoid,     TEXT("sigmoid(X)"));
		OP(Sign,        TEXT("sign(X)"));
		OP(Sin,         TEXT("sin(X)"));
		OP(Sinh,        TEXT("sinh(X)"));
		OP(Softplus,    TEXT("softplus(X)"));
		OP(Softsign,    TEXT("softsign(X)"));
		OP(Sqrt,        TEXT("sqrt(X)"));
		OP(Tan,         TEXT("tan(X)"));
		OP(Tanh,        TEXT("tanh(X)"));
#undef OP

		FString OpFunc = OpTable[(int32) OpType];

		if (OpFunc == FString(""))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Undefined ElementWise Unary operator name for operator:%d", int(OpType));
		}

		return OpFunc;
	}

	IMPLEMENT_GLOBAL_SHADER(TElementWiseUnaryCS, "/NNEHlslShaders/NNEHlslShadersElementWiseUnary.usf", "ElementWiseUnary", SF_Compute);
} // UE::NNEHlslShaders::Internal
