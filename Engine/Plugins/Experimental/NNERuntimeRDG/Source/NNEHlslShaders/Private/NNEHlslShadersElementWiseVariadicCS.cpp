// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersElementWiseVariadicCS.h"
#include "NNEHlslShadersLog.h"

namespace UE::NNEHlslShaders::Internal
{
	void TElementWiseVariadicCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FElementWiseVariadicConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);

		const FString OpFunc = GetOpFunc(PermutationVector.Get<FOperatorType>());

		OutEnvironment.SetDefine(TEXT("ELEMENTWISE_OP(X,Y)"), *OpFunc);
	}

	const FString TElementWiseVariadicCS::GetOpFunc(EElementWiseVariadicOperatorType OpType)
	{
		FString OpTable[(int32) EElementWiseVariadicOperatorType::MAX];

		for (int32 Idx = 0; Idx < (int32) EElementWiseVariadicOperatorType::MAX; ++Idx)
		{
			OpTable[Idx] = FString("");
		}

#define OP(OpName, OpFunc) OpTable[(int32) EElementWiseVariadicOperatorType::OpName] = OpFunc
		OP(Max, TEXT("max(X,Y)"));
		OP(Min, TEXT("min(X,Y)"));
		OP(Mean, TEXT("((X)+(Y))"));
		OP(Sum, TEXT("((X)+(Y))"));
#undef OP

		FString OpFunc = OpTable[(int32)OpType];

		if (OpFunc == "")
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Undefined ElementWise Variadic operator name for operator:%d", int(OpType));
		}

		return OpFunc;
	}

	IMPLEMENT_GLOBAL_SHADER(TElementWiseVariadicCS, "/NNEHlslShaders/NNEHlslShadersElementWiseVariadic.usf", "ElementWiseVariadic", SF_Compute);
} // UE::NNEHlslShaders::Internal
