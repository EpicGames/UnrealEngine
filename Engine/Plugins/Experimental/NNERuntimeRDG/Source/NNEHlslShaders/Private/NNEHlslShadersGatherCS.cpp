// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersGatherCS.h"
#include "NNETypes.h"

namespace UE::NNEHlslShaders::Internal
{
	// template <>
	void TGatherCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("DATA_TYPE"), 0);
		OutEnvironment.SetDefine(TEXT("INDICES_TYPE"), 0);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_DIMENSIONS"), FGatherConstants::MAX_NUM_DIMENSIONS);
		OutEnvironment.SetDefine(TEXT("NUM_GROUP_THREADS"), FGatherConstants::NUM_GROUP_THREADS);
	}

	//template <typename DataElementType, typename IndicesElementType>
	void TGatherCS::FillInParameters(int32 Axis, const NNE::FTensorShape& Data, const NNE::FTensorShape& Indices, FParameters& Parameters)
	{
		Parameters.Axis = Axis;

		TArray<int32> OutputShape;
		OutputShape.SetNum(Indices.Rank() + (Data.Rank() - 1));
		int32 index = 0;
		Parameters.OutputSize = 1;
		for (int32 i = 0; i < Axis; i++, index++)
		{
			OutputShape[index] = Data.GetData()[i];
			Parameters.OutputSize *= OutputShape[index];
		}
		for (int32 i = 0; i < Indices.Rank(); i++, index++)
		{
			OutputShape[index] = Indices.GetData()[i];
			Parameters.OutputSize *= OutputShape[index];
		}
		for (int32 i = (Axis + 1); i < Data.Rank(); i++, index++)
		{
			OutputShape[index] = Data.GetData()[i];
			Parameters.OutputSize *= OutputShape[index];
		}

		Parameters.NumDataDimensions = Data.Rank();
		Parameters.NumIndicesDimensions = Indices.Rank();

		int32 DataStride = 1;
		for (int32 i = Data.Rank() - 1; i >= 0; i--)
		{
			Parameters.DataStride_IndicesStride_OutputStride_DataSizes[i].X = DataStride;
			Parameters.OneDivDataStride_OneDivIndicesStride_OneDivOutputStride[i].X = 1.0 / (float)DataStride;
			DataStride *= Data.GetData()[i];
		}

		int32 IndicesStride = 1;
		for (int32 i = Indices.Rank() - 1; i >= 0; i--)
		{
			Parameters.DataStride_IndicesStride_OutputStride_DataSizes[i].Y = IndicesStride;
			Parameters.OneDivDataStride_OneDivIndicesStride_OneDivOutputStride[i].Y = 1.0 / (float)IndicesStride;
			IndicesStride *= Indices.GetData()[i];
		}

		int32 OutputStride = 1;
		for (int32 i = OutputShape.Num() - 1; i >= 0; i--)
		{
			Parameters.DataStride_IndicesStride_OutputStride_DataSizes[i].Z = OutputStride;
			Parameters.OneDivDataStride_OneDivIndicesStride_OneDivOutputStride[i].Z = 1.0 / (float)OutputStride;
			OutputStride *= OutputShape[i];
		}

		for (int32 Idx = 0; Idx < Data.Rank(); ++Idx)
		{
			Parameters.DataStride_IndicesStride_OutputStride_DataSizes[Idx].W = Data.GetData()[Idx];
		}
	}

	// template <typename DataElementType, typename IndicesElementType>
	FIntVector TGatherCS::GetGroupCount(const FParameters& Parameters)
	{
		return FIntVector(FMath::DivideAndRoundUp(Parameters.OutputSize, FGatherConstants::NUM_GROUP_THREADS), 1, 1);
	}

	//typedef TGatherCS<float, int32> FGatherFloatInt32CS;
	//IMPLEMENT_SHADER_TYPE(template<>, TGatherCS, TEXT("/NNE/GatherOp.usf"), TEXT("main"), SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(TGatherCS, "/NNEHlslShaders/NNEHlslShadersGather.usf", "Gather", SF_Compute);
	//template class TGatherCS<float, int32>;

} // UE::NNEHlslShaders::Internal