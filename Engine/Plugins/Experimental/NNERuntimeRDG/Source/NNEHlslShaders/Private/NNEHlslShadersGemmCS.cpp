// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersGemmCS.h"
#include "NNETypes.h"
#include "Algo/Accumulate.h"

namespace UE::NNEHlslShaders::Internal
{
	struct GemmMatrixParameters
	{
		uint32 M = 0;
		uint32 N = 0;
		uint32 K = 0;
		TArray<uint32> StackShapeA;
		TArray<uint32> StackShapeB;
		TArray<uint32> StackStrideA;
		TArray<uint32> StackStrideB;
	};

	static GemmMatrixParameters GetMatrixParametersMatMul(const NNE::FTensorShape& InputA, const NNE::FTensorShape& InputB)
	{
		check(InputA.Rank() != 0);
		check(InputB.Rank() != 0);

		GemmMatrixParameters Result;
		int32 NumStackDimensionsA = FMath::Max((int32)InputA.Rank() - 2, 0);
		int32 NumStackDimensionsB = FMath::Max((int32)InputB.Rank() - 2, 0);
		int32 NumStackDimensions = FMath::Max(NumStackDimensionsA, NumStackDimensionsB);

		check(NumStackDimensions <= FGemmConstants::MAX_NUM_STACK_DIMENSIONS);

		// Check matrix stack dimensions
		if (NumStackDimensionsA > 0 && NumStackDimensionsB > 0)
		{
			const int32 NumDimToCheck = FMath::Min(NumStackDimensionsA, NumStackDimensionsB);
			for (int32 i = 0; i < NumDimToCheck; i++)
			{
				const uint32 VolumeA = InputA.GetData()[NumStackDimensionsA - 1 - i];
				const uint32 VolumeB = InputB.GetData()[NumStackDimensionsB - 1 - i];

				check(VolumeA == 1 || VolumeB == 1 || VolumeA == VolumeB);
			}
		}

		const bool IsVectorA = InputA.Rank() == 1;
		const bool IsVectorB = InputB.Rank() == 1;

		Result.M = IsVectorA ? 1 : InputA.GetData()[InputA.Rank() - 2];
		Result.N = IsVectorB ? 1 : InputB.GetData()[InputB.Rank() - 1];
		Result.K = InputA.GetData()[IsVectorA ? 0 : InputA.Rank() - 1];
		check(InputB.GetData()[IsVectorB ? 0 : InputB.Rank() - 2] == Result.K);

		Result.StackShapeA.Init(1, NumStackDimensions);
		Result.StackShapeB.Init(1, NumStackDimensions);

		for (int32 i = 0; i < NumStackDimensions; i++)
		{
			int32 IdxA = InputA.Rank() - 3 - i;
			int32 IdxB = InputB.Rank() - 3 - i;

			Result.StackShapeA[Result.StackShapeA.Num() - 1 - i] = IdxA >= 0 ? InputA.GetData()[IdxA] : 1;
			Result.StackShapeB[Result.StackShapeB.Num() - 1 - i] = IdxB >= 0 ? InputB.GetData()[IdxB] : 1;
		}

		Result.StackStrideA.Init(1, NumStackDimensions);
		Result.StackStrideB.Init(1, NumStackDimensions);

		for (int32 i = NumStackDimensions - 2; i >= 0; i--)
		{
			Result.StackStrideA[i] = Result.StackStrideA[i + 1] * Result.StackShapeA[i + 1];
			Result.StackStrideB[i] = Result.StackStrideB[i + 1] * Result.StackShapeB[i + 1];
		}

		return Result;
	}

	void TGemmCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_STACK_DIMENSIONS"), FGemmConstants::MAX_NUM_STACK_DIMENSIONS);
	}

	void TGemmCS::FillInParameters(float Alpha, float Beta, int32 TransA, int32 TransB, const NNE::FTensorShape& InputA, const NNE::FTensorShape& InputB, const NNE::FTensorShape* InputC, float CScalar, TGemmCS::FParameters& Parameters)
	{
		check(InputA.Rank() == 2);
		check(InputB.Rank() == 2);

		uint32 M = TransA != 0 ? InputA.GetData()[1] : InputA.GetData()[0];
		uint32 K = TransA != 0 ? InputA.GetData()[0] : InputA.GetData()[1];
		uint32 N = TransB != 0 ? InputB.GetData()[0] : InputB.GetData()[1];
		check(K == (TransB != 0 ? InputB.GetData()[1] : InputB.GetData()[0]));

		Parameters.Alpha = Alpha;
		Parameters.Beta = Beta;
		Parameters.TransA = TransA;
		Parameters.TransB = TransB;
		Parameters.M = M;
		Parameters.N = N;
		Parameters.K = K;
		Parameters.MxK = M * K;
		Parameters.KxN = K * N;
		Parameters.MxN = M * N;
		Parameters.CWidth = 0;
		Parameters.CHeight = 0;
		if (InputC != nullptr)
		{
			Parameters.CWidth = InputC->Rank() == 0 ? 0 : InputC->GetData()[InputC->Rank() - 1];;
			Parameters.CHeight = InputC->Rank() == 0 ? 0 : InputC->Rank() == 1 ? 1 : InputC->GetData()[InputC->Rank() - 2];
		}
		
		Parameters.CScalar = CScalar;
	}

	void TGemmCS::FillInParametersMatMul(const NNE::FTensorShape& InputA, const NNE::FTensorShape& InputB, TGemmCS::FParameters& Parameters)
	{
		GemmMatrixParameters Params = GetMatrixParametersMatMul(InputA, InputB);

		Parameters.Alpha = 1.0f;
		Parameters.Beta = 1.0f;
		Parameters.TransA = 0;
		Parameters.TransB = 0;
		Parameters.M = Params.M;
		Parameters.N = Params.N;
		Parameters.K = Params.K;
		Parameters.MxK = Params.M * Params.K;
		Parameters.KxN = Params.K * Params.N;
		Parameters.MxN = Params.M * Params.N;
		Parameters.CWidth = 0;
		Parameters.CHeight = 0;
		Parameters.CScalar = 0.0f;

		for (int32 i = 0; i < Params.StackShapeA.Num(); i++)
		{
			Parameters.StackShapeA_StackShapeB_StackStrideA_StackStrideB[i] = FUint32Vector4(Params.StackShapeA[i], Params.StackShapeB[i], Params.StackStrideA[i], Params.StackStrideB[i]);
		}
	}

	FIntVector TGemmCS::GetGroupCount(const TGemmCS::FParameters& Parameters, EGemmAlgorithm Algorithm, int32 NumStackDimensions)
	{
		int32 OutputStackSize = 1;
		for (int32 i = 0; i < NumStackDimensions; i++)
		{
			OutputStackSize *= FMath::Max(
				Parameters.StackShapeA_StackShapeB_StackStrideA_StackStrideB[i].X,
				Parameters.StackShapeA_StackShapeB_StackStrideA_StackStrideB[i].Y);
		}

		int32 ThreadGroupCountValueZ = FMath::DivideAndRoundUp(OutputStackSize, 1);
		int32 ThreadGroupCountValueY = FMath::DivideAndRoundUp((int32)Parameters.M, 8);
		int32 ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 8);
		switch (Algorithm)
		{
		case EGemmAlgorithm::Simple8x8:
		case EGemmAlgorithm::SharedMemory8x8:
			ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 8);
			ThreadGroupCountValueY = FMath::DivideAndRoundUp((int32)Parameters.M, 8);
			break;
		case EGemmAlgorithm::Simple16x16:
		case EGemmAlgorithm::SharedMemory16x16:
		case EGemmAlgorithm::MultiWrite1x16:
		case EGemmAlgorithm::MultiWrite2x16:
			ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 16);
			ThreadGroupCountValueY = FMath::DivideAndRoundUp((int32)Parameters.M, 16);
			break;
		case EGemmAlgorithm::Simple32x32:
		case EGemmAlgorithm::SharedMemory32x32:
		case EGemmAlgorithm::MultiWrite1x32:
		case EGemmAlgorithm::MultiWrite2x32:
		case EGemmAlgorithm::MultiWrite4x32:
			ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 32);
			ThreadGroupCountValueY = FMath::DivideAndRoundUp((int32)Parameters.M, 32);
			break;
		case EGemmAlgorithm::Simple256x1:
			ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 256);
			ThreadGroupCountValueY = Parameters.M;
			break;
		case EGemmAlgorithm::MultiWrite2x64:
		case EGemmAlgorithm::MultiWrite4x64:
		case EGemmAlgorithm::MultiWrite8x64:
			ThreadGroupCountValueX = FMath::DivideAndRoundUp((int32)Parameters.N, 64);
			ThreadGroupCountValueY = FMath::DivideAndRoundUp((int32)Parameters.M, 64);
			break;
		default:
			break;
		}

		return FIntVector(ThreadGroupCountValueX, ThreadGroupCountValueY, ThreadGroupCountValueZ);
	}

	EGemmAlgorithm TGemmCS::GetAlgorithm(const TGemmCS::FParameters& Parameters)
	{
		return EGemmAlgorithm::MultiWrite1x32;
	}

	IMPLEMENT_GLOBAL_SHADER(TGemmCS, "/NNEHlslShaders/NNEHlslShadersGemm.usf", "Gemm", SF_Compute);
} // UE::NNEHlslShaders::Internal