// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersInstanceNormalizationCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void TInstanceNormalizationCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FInstanceNormalizationConstants::NUM_GROUP_THREADS);
	}

	IMPLEMENT_GLOBAL_SHADER(TInstanceNormalizationCS, "/NNEHlslShaders/NNEHlslShadersInstanceNormalization.usf", "InstanceNormalization", SF_Compute);
} // UE::NNEHlslShaders::Internal