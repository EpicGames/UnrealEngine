// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersLayerNormalizationCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void TLayerNormalizationCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), FLayerNormalizationConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	void TLayerNormalizationCS::FillInParameters(TConstArrayView<uint32> Shape, int32 Axis, float Epsilon, TLayerNormalizationCS::FParameters* Parameters)
	{
		check(Axis >= 0 && Axis < Shape.Num());
		check(Parameters);

		uint32 LayerSize = 1;
		for(int DimIdx = Axis; DimIdx < Shape.Num(); ++DimIdx)
		{
			LayerSize *= Shape[DimIdx];
		}

		Parameters->LayerSize = LayerSize;
		Parameters->Axis = Axis;
		Parameters->Epsilon = Epsilon;
	}

	IMPLEMENT_GLOBAL_SHADER(TLayerNormalizationCS, "/NNEHlslShaders/NNEHlslShadersLayerNormalization.usf", "LayerNormalization", SF_Compute);
} // UE::NNEHlslShaders::Internal