// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NNEHlslShadersLog.h"
#include "ShaderCore.h"

class FNNEHlslShadersModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("NNERuntimeRDG"));
		if (Plugin.IsValid())
		{
			const FString ShadersDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders/Private/NNEHlslShaders"));
			AddShaderSourceDirectoryMapping(TEXT("/NNEHlslShaders"), ShadersDir);
		}
		else
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Shaders directory not added. Failed to find NNERuntimeRDG plugin");
		}

	}

	virtual void ShutdownModule() override
	{

	}
};

IMPLEMENT_MODULE(FNNEHlslShadersModule, NNEHlslShaders);
