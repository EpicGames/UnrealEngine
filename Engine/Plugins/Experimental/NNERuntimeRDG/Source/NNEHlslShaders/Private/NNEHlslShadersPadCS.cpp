// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersPadCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FPadCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FPadConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	void FPadCS::LexFromString(EPadMode& OutValue, const TCHAR* StringVal)
	{
		OutValue = EPadMode::CONSTANT;
		if (FCString::Stricmp(StringVal, TEXT("CONSTANT")) == 0) OutValue = EPadMode::CONSTANT;
		else if (FCString::Stricmp(StringVal, TEXT("REFLECT")) == 0) OutValue = EPadMode::REFLECT;
		else if (FCString::Stricmp(StringVal, TEXT("EDGE")) == 0) OutValue = EPadMode::EDGE;
	}

	IMPLEMENT_GLOBAL_SHADER(FPadCS, "/NNEHlslShaders/NNEHlslShadersPad.usf", "Pad", SF_Compute);
} // UE::NNEHlslShaders::Internal