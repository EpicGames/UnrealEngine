// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersPoolCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FPoolCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FPoolConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FPoolCS, "/NNEHlslShaders/NNEHlslShadersPool.usf", "Pool", SF_Compute);
} // UE::NNEHlslShaders::Internal