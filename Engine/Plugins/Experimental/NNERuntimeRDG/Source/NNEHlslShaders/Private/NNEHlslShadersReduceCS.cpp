// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersReduceCS.h"
#include "NNE.h"
#include "RenderGraphBuilder.h"

namespace UE::NNEHlslShaders::Internal
{
	void TReduceCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), FReduceConstants::THREADGROUP_SIZE);
	}

	void TReduceCS::FillInParameters(TConstArrayView<uint32> Shape, int32 Axis, TReduceCS::FParameters* Parameters)
	{
		check(Axis >= 0 && Axis <= Shape.Num());
		check(Parameters);

		int32 NumElemBeforeAxis = 1;
		int32 NumElemAfterAxis = 1;
		for (int32 d = 0; d < Axis; ++d)
		{
			NumElemBeforeAxis *= Shape[d];
		}
		for (int32 d = Axis + 1; d < Shape.Num(); ++d)
		{
			NumElemAfterAxis *= Shape[d];
		}

		Parameters->NumElemBeforeAxis = NumElemBeforeAxis;
		Parameters->AxisSize = Shape[Axis];
		Parameters->NumElemAfterAxis = NumElemAfterAxis;
		Parameters->Epsilon = 0;
	}

	void TReduceCS::EnqueueRDG(FRDGBuilder& GraphBuilder, TReduceCS::FParameters* InParameters, FRDGBufferRef Input, FRDGBufferRef Output, EReduceOperatorType OperatorType, FRDGBufferRef Output2)
	{
		check(InParameters);

		InParameters->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input, PF_R32_FLOAT));
		InParameters->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output, PF_R32_FLOAT));
		if (Output2 != nullptr) {
			InParameters->Output2 = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output2, PF_R32_FLOAT));
		}

		TReduceCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<TReduceCS::FReduceType>(OperatorType);

		TShaderMapRef<TReduceCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

		FIntVector3 NumDispatches (
			1,
			(int32)InParameters->NumElemAfterAxis / GRHIMaxDispatchThreadGroupsPerDimension.Y + 1,
			(int32)InParameters->NumElemBeforeAxis / GRHIMaxDispatchThreadGroupsPerDimension.Z + 1
			);
		
		for(int32 DispatchIdxY = 0; DispatchIdxY < NumDispatches.Y; ++DispatchIdxY)
		{
			for(int32 DispatchIdxZ = 0; DispatchIdxZ < NumDispatches.Z; ++DispatchIdxZ)
			{
				FIntVector ThreadGroupCount {
					1,
					DispatchIdxY == NumDispatches.Y - 1 ? (int32)InParameters->NumElemAfterAxis % GRHIMaxDispatchThreadGroupsPerDimension.Y : GRHIMaxDispatchThreadGroupsPerDimension.Y,
					DispatchIdxZ == NumDispatches.Z - 1 ? (int32)InParameters->NumElemBeforeAxis % GRHIMaxDispatchThreadGroupsPerDimension.Z : GRHIMaxDispatchThreadGroupsPerDimension.Z
				};

				// Need to use a copy of InParameters to set DispatchIdxAndStride for a single pass
				TReduceCS::FParameters* CurParameters = GraphBuilder.AllocParameters(InParameters);
				CurParameters->DispatchIdxAndStride.X = (uint32) DispatchIdxY;
				CurParameters->DispatchIdxAndStride.Y = (uint32) DispatchIdxZ;
				CurParameters->DispatchIdxAndStride.Z = (uint32) GRHIMaxDispatchThreadGroupsPerDimension.Y;
				CurParameters->DispatchIdxAndStride.W = (uint32) GRHIMaxDispatchThreadGroupsPerDimension.Z;
				
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.Reduce.OneAxis.Dispatch.%d.%d", DispatchIdxY, DispatchIdxZ),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					CurParameters,
					ThreadGroupCount);
			}
		}
		
	}

	IMPLEMENT_GLOBAL_SHADER(TReduceCS, "/NNEHlslShaders/NNEHlslShadersReduce.usf", "Reduce", SF_Compute);
} // UE::NNEHlslShaders::Internal
