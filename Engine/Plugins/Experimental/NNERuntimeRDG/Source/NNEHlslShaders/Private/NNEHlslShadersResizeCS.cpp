// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersResizeCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FResizeCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		EMode Mode = PermutationVector.Get<FResizeCS::FMode>();
		
		//NOTE: cubic interpolation not currently supported
		if(Mode == EMode::Cubic)
		{
			return false;
		}

		return true;
	}

	void FResizeCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FResizeConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	ECoordTransMode FResizeCS::CoordTransModeFromString(const TCHAR* StringVal)
	{
		ECoordTransMode OutValue = ECoordTransMode::HalfPixel;
		if (FCString::Stricmp(StringVal, TEXT("half_pixel")) == 0) OutValue = ECoordTransMode::HalfPixel;
		else if (FCString::Stricmp(StringVal, TEXT("half_pixel_symmetric")) == 0) OutValue = ECoordTransMode::HalfPixelSymmetric;
		else if (FCString::Stricmp(StringVal, TEXT("pytorch_half_pixel")) == 0) OutValue = ECoordTransMode::PytorchHalfPixel;
		else if (FCString::Stricmp(StringVal, TEXT("align_corners")) == 0) OutValue = ECoordTransMode::AlignCorners;
		else if (FCString::Stricmp(StringVal, TEXT("asymmetric")) == 0) OutValue = ECoordTransMode::Asymmetric;
		else if (FCString::Stricmp(StringVal, TEXT("tf_half_pixel_for_nn")) == 0) OutValue = ECoordTransMode::TfHalfPixelForNn;
		else if (FCString::Stricmp(StringVal, TEXT("tf_crop_and_resize")) == 0) OutValue = ECoordTransMode::TfCropAndResize;

        return OutValue;
	}

	EMode FResizeCS::ModeFromString(const TCHAR* StringVal)
	{
		EMode OutValue = EMode::Nearest;
		if (FCString::Stricmp(StringVal, TEXT("nearest")) == 0) OutValue = EMode::Nearest;
		else if (FCString::Stricmp(StringVal, TEXT("linear")) == 0) OutValue = EMode::Linear;
		else if (FCString::Stricmp(StringVal, TEXT("cubic")) == 0) OutValue = EMode::Cubic;

        return OutValue;
	}

	ENearestMode FResizeCS::NearestModeFromString(const TCHAR* StringVal)
	{
		ENearestMode OutValue = ENearestMode::RoundPreferFloor;
		if (FCString::Stricmp(StringVal, TEXT("round_prefer_floor")) == 0) OutValue = ENearestMode::RoundPreferFloor;
		else if (FCString::Stricmp(StringVal, TEXT("round_prefer_ceil")) == 0) OutValue = ENearestMode::RoundPreferCeil;
		else if (FCString::Stricmp(StringVal, TEXT("floor")) == 0) OutValue = ENearestMode::Floor;
		else if (FCString::Stricmp(StringVal, TEXT("ceil")) == 0) OutValue = ENearestMode::Ceil;

        return OutValue;
	}

	IMPLEMENT_GLOBAL_SHADER(FResizeCS, "/NNEHlslShaders/NNEHlslShadersResize.usf", "Resize", SF_Compute);
} // UE::NNEHlslShaders::Internal