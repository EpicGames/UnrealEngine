// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersScatterNDCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FScatterNDCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), FScatterNDConstants::NUM_GROUP_THREADS);

	}

    EScatterNDReductionType FScatterNDCS::ReductionFromString(const TCHAR* StringVal)
	{
		EScatterNDReductionType OutValue = EScatterNDReductionType::None;
		if (FCString::Stricmp(StringVal, TEXT("none")) == 0) OutValue = EScatterNDReductionType::None;
		else if (FCString::Stricmp(StringVal, TEXT("add")) == 0) OutValue = EScatterNDReductionType::Add;
		else if (FCString::Stricmp(StringVal, TEXT("mul")) == 0) OutValue = EScatterNDReductionType::Mul;
		else if (FCString::Stricmp(StringVal, TEXT("max")) == 0) OutValue = EScatterNDReductionType::Max;
		else if (FCString::Stricmp(StringVal, TEXT("min")) == 0) OutValue = EScatterNDReductionType::Min;

        return OutValue;
	}

	IMPLEMENT_GLOBAL_SHADER(FScatterNDCS, "/NNEHlslShaders/NNEHlslShadersScatterND.usf", "ScatterND", SF_Compute);
} // UE::NNEHlslShaders::Internal