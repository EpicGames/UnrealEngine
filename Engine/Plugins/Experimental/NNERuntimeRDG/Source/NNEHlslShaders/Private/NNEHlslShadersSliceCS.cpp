// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersSliceCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FSliceCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FSliceConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FSliceCS, "/NNEHlslShaders/NNEHlslShadersSlice.usf", "Slice", SF_Compute);
} // UE::NNEHlslShaders::Internal