// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersSplitCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FSplitCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(InParameters))
		{
			return false;
		}

		const FPermutationDomain PermutationVector(InParameters.PermutationId);

		return PermutationVector.Get<FSplitCS::FSplitAxis>() < PermutationVector.Get<FSplitCS::FSplitRank>();
	}

	void FSplitCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FSplitConstants::NUM_GROUP_THREADS);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_SPLITS"), FSplitConstants::MAX_NUM_SPLITS);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_DIMENSIONS"), FSplitConstants::MAX_NUM_DIMENSIONS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FSplitCS, "/NNEHlslShaders/NNEHlslShadersSplit.usf", "Split", SF_Compute);
} // UE::NNEHlslShaders::Internal