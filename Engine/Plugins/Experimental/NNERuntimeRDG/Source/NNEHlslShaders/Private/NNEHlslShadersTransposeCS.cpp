// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersTransposeCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	void FTransposeCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FTransposeConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FTransposeCS, "/NNEHlslShaders/NNEHlslShadersTranspose.usf", "Transpose", SF_Compute);
} // UE::NNEHlslShaders::Internal