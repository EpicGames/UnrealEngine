// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersTypeHelper.h"

namespace UE::NNEHlslShaders::Internal
{
	const TCHAR* ShaderDataTypeToName(ENNEShaderDataType ShaderType)
	{
		switch (ShaderType)
		{
			case ENNEShaderDataType::BOOL	 : return TEXT("bool");
			case ENNEShaderDataType::FLOAT16 : return TEXT("float16_t");
			case ENNEShaderDataType::FLOAT32 : return TEXT("float");
			case ENNEShaderDataType::INT8	 : return TEXT("int8_t");
			case ENNEShaderDataType::INT16	 : return TEXT("int16_t");
			case ENNEShaderDataType::INT32	 : return TEXT("int32_t");
			case ENNEShaderDataType::UINT8	 : return TEXT("uint8_t");
			case ENNEShaderDataType::UINT16	 : return TEXT("uint16_t");
			case ENNEShaderDataType::UINT32	 : return TEXT("uint32_t");
			default:
				check(false);
				return nullptr;
		}
	}

	ENNEShaderDataType TensorToShaderDataType(ENNETensorDataType TensorType)
	{
		switch (TensorType)
		{
			case ENNETensorDataType::Boolean:    return ENNEShaderDataType::BOOL;
			case ENNETensorDataType::Half:		 return ENNEShaderDataType::FLOAT16;
			case ENNETensorDataType::Float:		 return ENNEShaderDataType::FLOAT32;
			case ENNETensorDataType::Int8:		 return ENNEShaderDataType::INT8;
			case ENNETensorDataType::Int16:		 return ENNEShaderDataType::INT16;
			case ENNETensorDataType::Int32:		 return ENNEShaderDataType::INT32;
			case ENNETensorDataType::UInt8:		 return ENNEShaderDataType::UINT8;
			case ENNETensorDataType::UInt16:	 return ENNEShaderDataType::UINT16;
			case ENNETensorDataType::UInt32:	 return ENNEShaderDataType::UINT32;
			default:
				check(false);
				return (ENNEShaderDataType)0;
		}
	}

	EPixelFormat TensorDataTypeToPixelFormat(ENNETensorDataType TensorType)
	{
		switch (TensorType)
		{
			case ENNETensorDataType::Boolean:    return EPixelFormat::PF_R8;
			case ENNETensorDataType::Half:		 return EPixelFormat::PF_R16F;
			case ENNETensorDataType::Float:		 return EPixelFormat::PF_R32_FLOAT;
			case ENNETensorDataType::Int8:		 return EPixelFormat::PF_R8_SINT;
			case ENNETensorDataType::Int16:		 return EPixelFormat::PF_R16_SINT;
			case ENNETensorDataType::Int32:		 return EPixelFormat::PF_R32_SINT;
			case ENNETensorDataType::UInt8:		 return EPixelFormat::PF_R8_UINT;
			case ENNETensorDataType::UInt16:	 return EPixelFormat::PF_R16_UINT;
			case ENNETensorDataType::UInt32:	 return EPixelFormat::PF_R32_UINT;
			default:
				check(false);
				return EPixelFormat::PF_Unknown;
		}
	}
} // UE::NNEHlslShaders::Internal
