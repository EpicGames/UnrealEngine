// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEHlslShadersUpsampleCS.h"
#include "NNE.h"

namespace UE::NNEHlslShaders::Internal
{
	bool FUpsampleCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!FHlslShaderBase::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		EUpsampleMode Mode = PermutationVector.Get<FUpsampleCS::FUpsampleMode>();
		const int32 NumDimensions = PermutationVector.Get<FUpsampleCS::FUpsampleNumDimensions>();

		return Mode == EUpsampleMode::Nearest ||
			(Mode == EUpsampleMode::Bilinear && NumDimensions > 1) ||
			(Mode == EUpsampleMode::Trilinear && NumDimensions > 2);
	}

	void FUpsampleCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), FUpsampleConstants::NUM_GROUP_THREADS);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
	}

	IMPLEMENT_GLOBAL_SHADER(FUpsampleCS, "/NNEHlslShaders/NNEHlslShadersUpsample.usf", "Upsample", SF_Compute);
} // UE::NNEHlslShaders::Internal