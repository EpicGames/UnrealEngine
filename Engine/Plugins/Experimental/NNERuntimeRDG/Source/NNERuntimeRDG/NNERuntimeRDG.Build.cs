// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class NNERuntimeRDG : ModuleRules
{
	public NNERuntimeRDG(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] 
		{ 
			"Core", 
			"CoreUObject", 
			"Engine", 
			"InputCore",
			"RenderCore"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"NNE",
			"NNEHlslShaders",
			"NNERuntimeRDGData",
			"RHI",
			"Projects",
			"TraceLog"
		});

		if (Target.Platform == UnrealTargetPlatform.Mac)
		{	
			PrivateDependencyModuleNames.Add("MetalRHI");
		}

		if (Target.Platform == UnrealTargetPlatform.Linux)
		{	
			PrivateDependencyModuleNames.Add("VulkanRHI");
		}

		if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
		{
			// Supported platforms (editor)
			if ( Target.Platform == UnrealTargetPlatform.Win64 || 
				Target.Platform == UnrealTargetPlatform.Linux || 
				Target.Platform == UnrealTargetPlatform.Mac
			)
			{
				PublicDefinitions.Add("WITH_NNE_RUNTIME_HLSL");
				PrivateDefinitions.Add("NNE_UTILITIES_AVAILABLE");
				PrivateDependencyModuleNames.Add("NNERuntimeRDGUtils");
			}
		}
		else
		{
			// Supported platforms (standalone)
			// NOTE: To add a supported platform:
			// - either here or in a NNERuntimeRDG_*.Build.cs extension, set WITH_NNE_RUNTIME_HLSL for Game/Client target type,
			// - set bSupportsNNEShaders=true in DataDrivenPlatformInfo.ini for the platform - shader format pair.
			if ( Target.Platform == UnrealTargetPlatform.Win64 || 
				 Target.Platform == UnrealTargetPlatform.Mac
			   )
			{
				PublicDefinitions.Add("WITH_NNE_RUNTIME_HLSL");
			}
		}

		PublicDefinitions.Add("NNERUNTIMERDGHLSL_BUFFER_LENGTH_ALIGNMENT=4");
	}
}
