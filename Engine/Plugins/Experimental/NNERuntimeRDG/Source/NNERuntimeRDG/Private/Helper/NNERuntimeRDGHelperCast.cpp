// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperCast.h"
#include "NNERuntimeRDGTensor.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Cast
{
	template<typename TIn, typename TOut> void ApplyWithResolvedTypes(TConstArrayView<TIn> InArray, TArray<TOut>& OutArray)
	{
		check(OutArray.IsEmpty());
		OutArray.Reset(InArray.Num());

		for (TIn e : InArray)
		{
			OutArray.Add((TOut)e);
		}
	}

	template<typename TOutput> void ApplyWithResolvedOutputType(const FTensor& Tensor, FTensor& OutputTensor)
	{
		ENNETensorDataType SourceType = Tensor.GetDataType();
		TArray<TOutput> OutputData;

		switch (SourceType)
		{
			case ENNETensorDataType::Float:
			{
				ApplyWithResolvedTypes(Tensor.GetPreparedData<float>(), OutputData);
				break;
			}
			case ENNETensorDataType::Half:
			{
				ApplyWithResolvedTypes(Tensor.GetPreparedData<FFloat16>(), OutputData);
				break;
			}
			case ENNETensorDataType::Int32:
			{
				ApplyWithResolvedTypes(Tensor.GetPreparedData<int32>(), OutputData);
				break;
			}
			case ENNETensorDataType::Int64:
			{
				ApplyWithResolvedTypes(Tensor.GetPreparedData<int64>(), OutputData);
				break;
			}
		}

		OutputTensor.SetPreparedData<TOutput>(OutputData);
	}

	void Apply(const FTensor& Tensor, FTensor& OutputTensor)
	{
		//Heuristic to avoid unexpected performance hit. This helper being intended for shape related arithmetic only.
		static constexpr int32 MaxItemInInputTensors = NNE::FTensorShape::MaxRank * 2;

		if (Tensor.HasPreparedData() && (Tensor.GetVolume() <= MaxItemInInputTensors))
		{

			ENNETensorDataType DestinationType = OutputTensor.GetDataType();
			switch (DestinationType)
			{
			case ENNETensorDataType::Float:
			{
				ApplyWithResolvedOutputType<float>(Tensor, OutputTensor);
				break;
			}
			case ENNETensorDataType::Half:
			{
				ApplyWithResolvedOutputType<FFloat16>(Tensor, OutputTensor);
				break;
			}
			case ENNETensorDataType::Int32:
			{
				ApplyWithResolvedOutputType<int32>(Tensor, OutputTensor);
				break;
			}
			case ENNETensorDataType::Int64:
			{
				ApplyWithResolvedOutputType<int64>(Tensor, OutputTensor);
				break;
			}
			}
		}
	}
} // UE::NNERuntimeRDG::Private::CPUHelper::Cast
