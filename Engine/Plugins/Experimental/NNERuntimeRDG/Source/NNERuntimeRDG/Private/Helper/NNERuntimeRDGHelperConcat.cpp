// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperConcat.h"

#include "NNERuntimeRDGTensorIdxIterator.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Concat
{
	void Apply(TConstArrayView<FTensorRef> InputTensors, FTensor& OutputTensor, int32 Axis)
	{
		static constexpr int32 MaxItemInOutputTensor = NNE::FTensorShape::MaxRank * 2;
		
		if (OutputTensor.GetVolume() >= MaxItemInOutputTensor)
		{
			return;
		}

		if (OutputTensor.GetDataType() != ENNETensorDataType::Float)
		{
			return;
		}
		
		for (FTensorRef InputTensor : InputTensors)
		{
			check(InputTensor != nullptr);
			if (!InputTensor->HasPreparedData())
			{
				return;
			}
		}

		check(Axis >= 0 && Axis < OutputTensor.GetShape().Rank());

		TArray<float> OutputData;
		const Private::TensorIdxIterator itOutput(OutputTensor.GetShape());
		int32 ConcatAxisOffset = 0; 

		OutputData.SetNumUninitialized(OutputTensor.GetVolume());
		
		for (FTensorRef InputTensor : InputTensors)
		{
			TConstArrayView<float> InputData = InputTensor->GetPreparedData<float>();
			Private::TensorIdxIterator itInput(InputTensor->GetShape());

			do
			{
				TArray<uint32> CurOutputPosition(itInput.GetPositions());
				CurOutputPosition[Axis] += ConcatAxisOffset;
				
				float Value = InputData[itInput.GetIndex()];
				OutputData[itOutput.GetIndexFromPosition(CurOutputPosition)] = Value;
				
			} while (itInput.Advance());
			
			ConcatAxisOffset += InputTensor->GetShape().GetData()[Axis];
		}

		OutputTensor.SetPreparedData<float>(OutputData);
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::Concat
