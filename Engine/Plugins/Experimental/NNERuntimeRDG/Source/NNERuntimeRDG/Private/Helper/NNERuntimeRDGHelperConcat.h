// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "NNERuntimeRDGTensor.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Concat
{
	void NNERUNTIMERDG_API Apply(TConstArrayView<FTensorRef> InputTensors, FTensor& OutputTensor, int32 Axis);

} // UE::NNERuntimeRDG::Private::CPUHelper::Concat