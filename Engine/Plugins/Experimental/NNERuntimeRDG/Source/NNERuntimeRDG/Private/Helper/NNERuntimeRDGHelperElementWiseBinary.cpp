// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperElementWiseBinary.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGTensorIdxIterator.h"
#include "NNETypes.h"
#include "Math/UnrealMathUtility.h"
#include "MathUtil.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseBinary
{
	using EElementWiseBinaryOperatorType = UE::NNEHlslShaders::Internal::EElementWiseBinaryOperatorType;

	template<EElementWiseBinaryOperatorType OpType, typename TData> TData Apply(TData X, TData Y);

	template<> float Apply<EElementWiseBinaryOperatorType::Add, float>(float X, float Y) { return X + Y; }
	template<> float Apply<EElementWiseBinaryOperatorType::Div, float>(float X, float Y) { return X / Y; }
	template<> float Apply<EElementWiseBinaryOperatorType::Mod, float>(float X, float Y) { return FMath::Fmod(X, Y); }
	template<> float Apply<EElementWiseBinaryOperatorType::Mul, float>(float X, float Y) { return X * Y; }
	template<> float Apply<EElementWiseBinaryOperatorType::Prelu, float>(float X, float Y) { return (X < 0.0f) ? (Y * X) : X; }
	template<> float Apply<EElementWiseBinaryOperatorType::Pow, float>(float X, float Y) { return FMath::Pow(X, Y); }
	template<> float Apply<EElementWiseBinaryOperatorType::Sub, float>(float X, float Y) { return X - Y; }

	template<> int32 Apply<EElementWiseBinaryOperatorType::Add, int32>(int32 X, int32 Y) { return X + Y; }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Div, int32>(int32 X, int32 Y) { return X / Y; }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Mod, int32>(int32 X, int32 Y) { return X % Y; }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Mul, int32>(int32 X, int32 Y) { return X * Y; }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Prelu, int32>(int32 X, int32 Y) { return (X < 0.0f) ? (Y * X) : X; }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Pow, int32>(int32 X, int32 Y) { return (int32)FMath::Pow((float)X, (float)Y); }
	template<> int32 Apply<EElementWiseBinaryOperatorType::Sub, int32>(int32 X, int32 Y) { return X - Y; }

	template<> int64 Apply<EElementWiseBinaryOperatorType::Add, int64>(int64 X, int64 Y) { return X + Y; }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Div, int64>(int64 X, int64 Y) { return X / Y; }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Mod, int64>(int64 X, int64 Y) { return X % Y; }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Mul, int64>(int64 X, int64 Y) { return X * Y; }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Prelu, int64>(int64 X, int64 Y) { return (X < 0.0f) ? (Y * X) : X; }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Pow, int64>(int64 X, int64 Y) { return (int64)FMath::Pow((float)X, (float)Y); }
	template<> int64 Apply<EElementWiseBinaryOperatorType::Sub, int64>(int64 X, int64 Y) { return X - Y; }

	template<EElementWiseBinaryOperatorType OpType, typename TData> void Apply(const FTensor& LHSTensor, const FTensor& RHSTensor, FTensor& OutputTensor)
	{
		//Heuristic to avoid unexpected performance hit. This helper being intended for shape related arithmetic only.
		static constexpr int32 MaxItemInInputTensors = NNE::FTensorShape::MaxRank * 2;

		if (LHSTensor.HasPreparedData() && 
			RHSTensor.HasPreparedData() && 
			(LHSTensor.GetVolume() <= MaxItemInInputTensors) &&
			(RHSTensor.GetVolume() <= MaxItemInInputTensors))
		{
			TConstArrayView<TData> LHSData = LHSTensor.GetPreparedData<TData>();
			TConstArrayView<TData> RHSData = RHSTensor.GetPreparedData<TData>();
			TArray<TData> OutputData;
			OutputData.Reserve(OutputTensor.GetVolume());

			Private::TensorIdxIterator it(OutputTensor.GetShape());
			do
			{
				int32 LHSIdx = it.GetIndexToBroadcastedShape(LHSTensor.GetShape());
				int32 RHSIdx = it.GetIndexToBroadcastedShape(RHSTensor.GetShape());
				TData LHSValue = LHSData.GetData()[LHSIdx];
				TData RHSValue = RHSData.GetData()[RHSIdx];
				OutputData.Add(Apply<OpType, TData>(LHSValue, RHSValue));
			} while (it.Advance());

			check(OutputData.Num() == OutputTensor.GetVolume());
			OutputTensor.SetPreparedData<TData>(OutputData);
		}
	}

	template<typename TData> void ApplyResolvedDataType(EElementWiseBinaryOperatorType OpType, const FTensor& LHSTensor, const FTensor& RHSTensor, FTensor& OutputTensor)
	{
		switch (OpType)
		{
		case EElementWiseBinaryOperatorType::Add:
			Apply<EElementWiseBinaryOperatorType::Add, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Div:
			Apply<EElementWiseBinaryOperatorType::Div, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Mod:
			Apply<EElementWiseBinaryOperatorType::Mod, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Mul:
			Apply<EElementWiseBinaryOperatorType::Mul, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Prelu:
			Apply<EElementWiseBinaryOperatorType::Prelu, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Pow:
			Apply<EElementWiseBinaryOperatorType::Pow, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		case EElementWiseBinaryOperatorType::Sub:
			Apply<EElementWiseBinaryOperatorType::Sub, TData>(LHSTensor, RHSTensor, OutputTensor);
			break;
		default:
			break;
		}
	}

	void Apply(EElementWiseBinaryOperatorType OpType, const FTensor& LHSTensor, const FTensor& RHSTensor, FTensor& OutputTensor)
	{
		switch (OutputTensor.GetDataType())
		{
		case ENNETensorDataType::Float:
			ApplyResolvedDataType<float>(OpType, LHSTensor, RHSTensor, OutputTensor);
			break;
		case ENNETensorDataType::Int32:
			ApplyResolvedDataType<int32>(OpType, LHSTensor, RHSTensor, OutputTensor);
			break;
		case ENNETensorDataType::Int64:
			ApplyResolvedDataType<int64>(OpType, LHSTensor, RHSTensor, OutputTensor);
			break;
		default:
			break;
		}
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseBinary
