// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "NNEHlslShadersOperator.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

namespace UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseBinary
{
	void NNERUNTIMERDG_API Apply(UE::NNEHlslShaders::Internal::EElementWiseBinaryOperatorType OpType, const FTensor& LHSTensor, const FTensor& RHSTensor, FTensor& OutputTensor);
	
} // UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseBinary