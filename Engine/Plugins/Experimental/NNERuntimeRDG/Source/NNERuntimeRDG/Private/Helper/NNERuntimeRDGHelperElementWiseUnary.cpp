// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperElementWiseUnary.h"

#include "Math/UnrealMathUtility.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseUnary
{
	using EElementWiseUnaryOperatorType = UE::NNEHlslShaders::Internal::EElementWiseUnaryOperatorType;

	template<EElementWiseUnaryOperatorType OpType> float Apply(float X, float Alpha, float Beta, float Gamma);
	
	template<> float Apply<EElementWiseUnaryOperatorType::Abs>(float X, float Alpha, float Beta, float Gamma) { return FMath::Abs(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Acos>(float X, float Alpha, float Beta, float Gamma) { return FMath::Acos(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Acosh>(float X, float Alpha, float Beta, float Gamma) {
		//https://mathworld.wolfram.com/InverseHyperbolicCosine.html
		float FloatNan = FMath::Sqrt(-1.0f);
		float yAboveOne = FMath::Loge(X + FMath::Sqrt(X + 1.0f) * FMath::Sqrt(X - 1.0f));
		if (X == 1.0f)
		{
			return 0.0f;
		}
		else
		{
			return (X >= 1.0f) ? yAboveOne : FloatNan;
		}
	}
	
	template<> float Apply<EElementWiseUnaryOperatorType::Asin>(float X, float Alpha, float Beta, float Gamma) { return FMath::Asin(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Asinh>(float X, float Alpha, float Beta, float Gamma) {
		//https://mathworld.wolfram.com/InverseHyperbolicSine.html
		return FMath::Loge(X + FMath::Sqrt(1 + (X * X)));
	}
	
	template<> float Apply<EElementWiseUnaryOperatorType::Atan>(float X, float Alpha, float Beta, float Gamma) { return FMath::Atan(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Atanh>(float X, float Alpha, float Beta, float Gamma) {
		//https://mathworld.wolfram.com/InverseHyperbolicTangent.html
		return 0.5f * (FMath::Loge(1 + X) - FMath::Loge(1 - X));
	}

	template<> float Apply<EElementWiseUnaryOperatorType::Ceil>(float X, float Alpha, float Beta, float Gamma) { return FMath::CeilToFloat(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Clip>(float X, float Alpha, float Beta, float Gamma) { return FMath::Clamp(X, Alpha, Beta); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Cos>(float X, float Alpha, float Beta, float Gamma) { return FMath::Cos(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Cosh>(float X, float Alpha, float Beta, float Gamma) {
		//https://mathworld.wolfram.com/HyperbolicCosine.html
		return 0.5f * (FMath::Exp(X) + FMath::Exp(-X));
	}
	
	template<> float Apply<EElementWiseUnaryOperatorType::Elu>(float X, float Alpha, float Beta, float Gamma) {
		//https://github.com/onnx/onnx/blob/main/docs/Operators.md#elu
		float yNeg = Alpha * (FMath::Exp(X) - 1.0f);
		float yPosOrZero = X;
		return (X >= 0.0f) ? yPosOrZero : yNeg;
	}
	
	template<> float Apply<EElementWiseUnaryOperatorType::Exp>(float X, float Alpha, float Beta, float Gamma) { return FMath::Exp(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Floor>(float X, float Alpha, float Beta, float Gamma) { return FMath::Floor(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::IsInf>(float X, float Alpha, float Beta, float Gamma) { return !FMath::IsFinite(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::IsNan>(float X, float Alpha, float Beta, float Gamma) { return FMath::IsNaN(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::HardSigmoid>(float X, float Alpha, float Beta, float Gamma) {
		//https://github.com/onnx/onnx/blob/main/docs/Operators.md#hardSigmoid
		return FMath::Max(0.0f, FMath::Min(1.0f, Alpha * X + Beta));
	}

	template<> float Apply<EElementWiseUnaryOperatorType::HardSwish>(float X, float Alpha, float Beta, float Gamma) {
		//https://github.com/onnx/onnx/blob/main/docs/Operators.md#hardSwish
		return Apply<EElementWiseUnaryOperatorType::HardSigmoid>(X, 1.0f / 6.0f, 0.5f, Gamma);
	}

	template<> float Apply<EElementWiseUnaryOperatorType::LeakyRelu>(float X, float Alpha, float Beta, float Gamma) { return (X >= 0.0f) ? X : Alpha * X; }

	template<> float Apply<EElementWiseUnaryOperatorType::Log>(float X, float Alpha, float Beta, float Gamma) { return FMath::Loge(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Neg>(float X, float Alpha, float Beta, float Gamma) { return -X; }

	template<> float Apply<EElementWiseUnaryOperatorType::Reciprocal>(float X, float Alpha, float Beta, float Gamma) { return 1.0f/X; }

	template<> float Apply<EElementWiseUnaryOperatorType::Relu>(float X, float Alpha, float Beta, float Gamma) { return FMath::Max(X, 0.0f); }

	template<> float Apply<EElementWiseUnaryOperatorType::Round>(float X, float Alpha, float Beta, float Gamma) { return FMath::RoundToFloat(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Selu>(float X, float Alpha, float Beta, float Gamma) {
		//https://github.com/onnx/onnx/blob/main/docs/Operators.md#Selu
		float yNegOrZero = Gamma * (Alpha * FMath::Exp(X) - Alpha);
		float yPos = Gamma * X;
		return (X > 0.0f) ? yPos : yNegOrZero;
	}
	
	template<> float Apply<EElementWiseUnaryOperatorType::Sigmoid>(float X, float Alpha, float Beta, float Gamma) { return 1.0f / (1.0f + FMath::Exp(-X)); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Sign>(float X, float Alpha, float Beta, float Gamma) { return FMath::Sign(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Sin>(float X, float Alpha, float Beta, float Gamma) { return FMath::Sin(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Sinh>(float X, float Alpha, float Beta, float Gamma) { return FMath::Sinh(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Softplus>(float X, float Alpha, float Beta, float Gamma) { return FMath::Loge(FMath::Exp(X) + 1.0f); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Softsign>(float X, float Alpha, float Beta, float Gamma) { return X / (1.0f + FMath::Abs(X)); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Sqrt>(float X, float Alpha, float Beta, float Gamma) { return FMath::Sqrt(X); }
	
	template<> float Apply<EElementWiseUnaryOperatorType::Tan>(float X, float Alpha, float Beta, float Gamma) { return FMath::Tan(X); }

	template<> float Apply<EElementWiseUnaryOperatorType::Tanh>(float X, float Alpha, float Beta, float Gamma) {
		//https://mathworld.wolfram.com/HyperbolicTangent.html
		float SinhValue = Apply<EElementWiseUnaryOperatorType::Sinh>(X, Alpha, Beta, Gamma);
		float CoshValue = Apply<EElementWiseUnaryOperatorType::Cosh>(X, Alpha, Beta, Gamma);
		return SinhValue / CoshValue;
	}

	template<> float Apply<EElementWiseUnaryOperatorType::Erf>(float X, float Alpha, float Beta, float Gamma) {
		//https://aapt.scitation.org/doi/abs/10.1119/1.15018?journalCode=ajp
		float a = 167.0f / 148.0f;
		float b = 11.0f / 109.0f;
		float x3 = X * X * X;
		return Apply<EElementWiseUnaryOperatorType::Tanh>(a * X + b * x3, Alpha, Beta, Gamma);
	}

	template<EElementWiseUnaryOperatorType OpType> void Apply(const FTensor& Tensor, float Alpha, float Beta, float Gamma, FTensor& OutputTensor)
	{
		//Heuristic to avoid unexpected performance hit. This helper being intended for shape related arithmetic only.
		static constexpr int32 MaxItemInInputTensors = NNE::FTensorShape::MaxRank * 2;

		if (Tensor.HasPreparedData() && (Tensor.GetVolume() <= MaxItemInInputTensors))
		{
			switch (Tensor.GetDataType())
			{
				case ENNETensorDataType::Float:
				{
					TConstArrayView<float> TensorData = Tensor.GetPreparedData<float>();
					TArray<float> OutputData;
					OutputData.Reserve(TensorData.Num());
					for (float elem : TensorData)
					{
						OutputData.Add(Apply<OpType>(elem, Alpha, Beta, Gamma));
					}
					OutputTensor.SetPreparedData<float>(OutputData);
				} break;
				case ENNETensorDataType::Half:
				{
					TConstArrayView<FFloat16> TensorData = Tensor.GetPreparedData<FFloat16>();
					TArray<FFloat16> OutputData;
					OutputData.Reserve(TensorData.Num());
					for (FFloat16 elem : TensorData)
					{
						OutputData.Add(Apply<OpType>(elem, Alpha, Beta, Gamma));
					}
					OutputTensor.SetPreparedData<FFloat16>(OutputData);
				} break;
				default:
					break;
			}
		}
	}

	void Apply(EElementWiseUnaryOperatorType OpType, const FTensor& Tensor, float Alpha, float Beta, float Gamma, FTensor& OutputTensor)
	{
		switch (OpType)
		{
		case EElementWiseUnaryOperatorType::Abs:
			Apply<EElementWiseUnaryOperatorType::Abs>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Acos:
			Apply<EElementWiseUnaryOperatorType::Acos>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Acosh:
			Apply<EElementWiseUnaryOperatorType::Acosh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Asin:
			Apply<EElementWiseUnaryOperatorType::Asin>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Asinh:
			Apply<EElementWiseUnaryOperatorType::Asinh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Atan:
			Apply<EElementWiseUnaryOperatorType::Atan>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Atanh:
			Apply<EElementWiseUnaryOperatorType::Atanh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Ceil:
			Apply<EElementWiseUnaryOperatorType::Ceil>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Clip:
			Apply<EElementWiseUnaryOperatorType::Clip>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Cos:
			Apply<EElementWiseUnaryOperatorType::Cos>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Cosh:
			Apply<EElementWiseUnaryOperatorType::Cosh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Elu:
			Apply<EElementWiseUnaryOperatorType::Elu>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Erf:
			Apply<EElementWiseUnaryOperatorType::Erf>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Exp:
			Apply<EElementWiseUnaryOperatorType::Exp>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Floor:
			Apply<EElementWiseUnaryOperatorType::Floor>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::IsInf:
			Apply<EElementWiseUnaryOperatorType::IsInf>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::IsNan:
			Apply<EElementWiseUnaryOperatorType::IsNan>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::HardSigmoid:
			Apply<EElementWiseUnaryOperatorType::HardSigmoid>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::HardSwish:
			Apply<EElementWiseUnaryOperatorType::HardSwish>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::LeakyRelu:
			Apply<EElementWiseUnaryOperatorType::LeakyRelu>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Log:
			Apply<EElementWiseUnaryOperatorType::Log>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Neg:
			Apply<EElementWiseUnaryOperatorType::Neg>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Reciprocal:
			Apply<EElementWiseUnaryOperatorType::Reciprocal>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Relu:
			Apply<EElementWiseUnaryOperatorType::Relu>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Round:
			Apply<EElementWiseUnaryOperatorType::Round>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Selu:
			Apply<EElementWiseUnaryOperatorType::Selu>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Sigmoid:
			Apply<EElementWiseUnaryOperatorType::Sigmoid>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Sign:
			Apply<EElementWiseUnaryOperatorType::Sign>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Sin:
			Apply<EElementWiseUnaryOperatorType::Sin>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Sinh:
			Apply<EElementWiseUnaryOperatorType::Sinh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Softplus:
			Apply<EElementWiseUnaryOperatorType::Softplus>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Softsign:
			Apply<EElementWiseUnaryOperatorType::Softsign>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Sqrt:
			Apply<EElementWiseUnaryOperatorType::Sqrt>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Tan:
			Apply<EElementWiseUnaryOperatorType::Tan>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		case EElementWiseUnaryOperatorType::Tanh:
			Apply<EElementWiseUnaryOperatorType::Tanh>(Tensor, Alpha, Beta, Gamma, OutputTensor);
			break;
		default:
			break;
		}
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseUnary
