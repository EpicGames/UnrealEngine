// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "NNEHlslShadersOperator.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

namespace UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseUnary
{
	void NNERUNTIMERDG_API Apply(UE::NNEHlslShaders::Internal::EElementWiseUnaryOperatorType OpType, const FTensor& Tensor, float Alpha, float Beta, float Gamma, FTensor& OutputTensor);

} // UE::NNERuntimeRDG::Private::CPUHelper::ElementWiseUnary