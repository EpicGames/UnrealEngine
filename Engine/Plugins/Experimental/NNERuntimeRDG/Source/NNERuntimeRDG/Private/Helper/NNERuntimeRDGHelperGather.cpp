// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperGather.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGTensorIdxIterator.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Gather
{
	template <typename TData, typename TInd> void Gather1DResolvedTypes(const FTensor& DataTensor, const FTensor& IndicesTensor, FTensor& OutputTensor)
	{
		TConstArrayView<TData> DataData = DataTensor.GetPreparedData<TData>();
		TConstArrayView<TInd> IndicesData = IndicesTensor.GetPreparedData<TInd>();
		TArray<TData> OutputData;

		check(IndicesData.Num() == OutputTensor.GetVolume());
		OutputData.Reserve(IndicesData.Num());

		for (TInd indices : IndicesData)
		{
			OutputData.Add(DataData[indices]);
		}

		OutputTensor.SetPreparedData<TData>(OutputData);
	}

	template <typename TInd> void Gather1DResolvedIndices(const FTensor& DataTensor, const FTensor& IndicesTensor, FTensor& OutputTensor)
	{
		switch (DataTensor.GetDataType())
		{
		case ENNETensorDataType::Float:
			Gather1DResolvedTypes<float, TInd>(DataTensor, IndicesTensor, OutputTensor);
			break;
		case ENNETensorDataType::Int32:
			Gather1DResolvedTypes<int32, TInd>(DataTensor, IndicesTensor, OutputTensor);
			break;
		case ENNETensorDataType::Int64:
			Gather1DResolvedTypes<int64, TInd>(DataTensor, IndicesTensor, OutputTensor);
			break;
		}
	}

	void Apply(const FTensor& DataTensor, const FTensor& IndicesTensor, int32 Axis, FTensor& OutputTensor)
	{
		static constexpr int32 MaxItemInOutputTensor = NNE::FTensorShape::MaxRank * 2;

		if (OutputTensor.GetVolume() >= MaxItemInOutputTensor)
		{
			return;
		}

		if ((DataTensor.GetDataType() != ENNETensorDataType::Float) &&
			(DataTensor.GetDataType() != ENNETensorDataType::Int64) &&
			(DataTensor.GetDataType() != ENNETensorDataType::Int32))
		{
			return;
		}
		check(OutputTensor.GetDataType() == DataTensor.GetDataType());

		if ((IndicesTensor.GetDataType() != ENNETensorDataType::Int32) &&
			(IndicesTensor.GetDataType() != ENNETensorDataType::Int64))
		{
			return;
		}

		if (!DataTensor.HasPreparedData() || !IndicesTensor.HasPreparedData())
		{
			return;
		}

		if (DataTensor.GetShape().Rank() > 1 || IndicesTensor.GetShape().Rank() > 1)
		{
			return;
		}

		check(Axis == 0);

		if (IndicesTensor.GetDataType() == ENNETensorDataType::Int64)
		{
			Gather1DResolvedIndices<int64>(DataTensor, IndicesTensor, OutputTensor);
		}
		else
		{
			Gather1DResolvedIndices<int32>(DataTensor, IndicesTensor, OutputTensor);
		}
		
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::Gather
