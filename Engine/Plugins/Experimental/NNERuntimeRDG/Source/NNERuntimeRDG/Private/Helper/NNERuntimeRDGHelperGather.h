// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

namespace UE::NNERuntimeRDG::Private::CPUHelper::Gather
{
	void NNERUNTIMERDG_API Apply(const FTensor& DataTensor, const FTensor& IndicesTensor, int32 Axis, FTensor& OutputTensor);

} // UE::NNERuntimeRDG::Private::CPUHelper::Gather