// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Algo/Transform.h"
#include "Algo/Count.h"
#include "CoreMinimal.h"
#include "Misc/EnumerateRange.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::ShapeHelper::Reshape
{
	template<typename DataType, class OutputShapeType >
	bool ReshapeTensor(
		const NNE::FTensorShape& InputTensorShape,
		const FTensor& ShapeTensor,
		bool bAllowZero,
		OutputShapeType& OutShape)
	{

		TArray<DataType, TInlineAllocator<NNE::FTensorShape::MaxRank>> ReshapedShape(ShapeTensor.GetPreparedData<DataType>());

		if (!bAllowZero)
		{
			// at most 1 dimension can be -1
			if (Algo::Count(ReshapedShape, -1) > 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Reshape: Shape tensor can't contain more than one '-1'.");
				return false;
			}
			for (TEnumerateRef<DataType> Elem : EnumerateRange(ReshapedShape))
			{
				if (!(*Elem != 0 || InputTensorShape.Rank() > Elem.GetIndex()))
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Reshape: Shape tensor contains '0' in an invalid place.");
					return false;
				}
				*Elem =
					*Elem == 0 ?
					InputTensorShape.GetData()[Elem.GetIndex()]
					:
					*Elem;
			}
		}
		else
		{
			// no -1 is allowed if there is a 0
			if (Algo::Count(ReshapedShape, 0) != 0 && Algo::Count(ReshapedShape, -1) != 0)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Reshape: Shape tensor contains both '0' and '-1'. This is not allowed.");
				return false;
			}
		}

		auto PartialVolume = [](TConstArrayView<DataType> Shape) -> uint32
		{
			uint32 Volume = 1;
			for (DataType Dim : Shape)
			{
				Volume *= Dim != -1 ? (DataType)Dim : 1;
			}
			return Volume;
		};

		OutShape.SetNumUninitialized(ReshapedShape.Num());

		for (TConstEnumerateRef<DataType> Elem : EnumerateRange(ReshapedShape))
		{
			OutShape[Elem.GetIndex()] =
				*Elem == -1 ?
				(uint32)InputTensorShape.Volume() / PartialVolume(ReshapedShape)
				:
				(uint32)*Elem;
		}

		return true;
	}

} // UE::NNERuntimeRDG::Private::ShapeHelper::Reshape
