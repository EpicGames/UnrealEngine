// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperSlice.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGTensorIdxIterator.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Slice
{
	template<typename TData> void ApplyResolvedInputType(const FTensor& InputTensor, FTensor& OutputTensor, TConstArrayView<int32> Starts, TConstArrayView<int32> Steps)
	{
		check(InputTensor.HasPreparedData());
		check(InputTensor.GetShape().Rank() == Starts.Num());
		check(OutputTensor.GetShape().Rank() == Starts.Num());
		check(InputTensor.GetShape().Rank() == Steps.Num());

		TArray<TData> OutputData;
		TConstArrayView<TData> InputData = InputTensor.GetPreparedData<TData>();
		Private::TensorIdxIterator ItOutput(OutputTensor.GetShape());
		const Private::TensorIdxIterator ItInput(InputTensor.GetShape());

		OutputData.SetNumUninitialized(OutputTensor.GetVolume());
		do
		{
			TArray<uint32> CurInputPosition(ItOutput.GetPositions());
			for (int DimIdx = 0; DimIdx < CurInputPosition.Num(); ++DimIdx)
			{
				CurInputPosition[DimIdx] = Starts[DimIdx] + (int32) CurInputPosition[DimIdx] * Steps[DimIdx];
			}

			TData Value = InputData[ItInput.GetIndexFromPosition(CurInputPosition)];
			OutputData[ItOutput.GetIndex()] = Value;

		} while (ItOutput.Advance());

		OutputTensor.SetPreparedData<TData>(OutputData);
	}

	void Apply(const FTensor& InputTensor, FTensor& OutputTensor, TConstArrayView<int32> Starts, TConstArrayView<int32> Steps)
	{
		static constexpr int32 MaxItemInOutputTensor = NNE::FTensorShape::MaxRank * 2;

		if (OutputTensor.GetVolume() >= MaxItemInOutputTensor)
		{
			return;
		}

		if (!InputTensor.HasPreparedData())
		{
			return;
		}

		switch (InputTensor.GetDataType())
		{
		case ENNETensorDataType::Int32:
			ApplyResolvedInputType<int32>(InputTensor, OutputTensor, Starts, Steps);
			break;

		case ENNETensorDataType::Int64:
			ApplyResolvedInputType<int64>(InputTensor, OutputTensor, Starts, Steps);
			break;

		case ENNETensorDataType::Float:
			ApplyResolvedInputType<float>(InputTensor, OutputTensor, Starts, Steps);
			break;
		}
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::Slice
