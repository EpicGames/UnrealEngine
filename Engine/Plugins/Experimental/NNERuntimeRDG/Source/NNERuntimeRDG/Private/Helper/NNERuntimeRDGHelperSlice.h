// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

namespace UE::NNERuntimeRDG::Private::CPUHelper::Slice
{
	void NNERUNTIMERDG_API Apply(const FTensor& InputTensor, FTensor& OutputTensor, TConstArrayView<int32> Starts, TConstArrayView<int32> Steps);

} // UE::NNERuntimeRDG::Private::CPUHelper::Slice