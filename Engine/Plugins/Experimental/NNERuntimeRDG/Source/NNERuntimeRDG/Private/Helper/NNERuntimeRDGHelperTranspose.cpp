// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHelperTranspose.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGTensorIdxIterator.h"

namespace UE::NNERuntimeRDG::Private::CPUHelper::Transpose
{
	bool Apply(const FTensor& Tensor, TConstArrayView<int32> Perms, FTensor& OutputTensor)
	{
		if (!Tensor.HasPreparedData())
		{
			return false;
		}
		if (Tensor.GetDataType() != ENNETensorDataType::Float)
		{
			return false;
		}

		TConstArrayView<uint32> TensorShape = Tensor.GetShape().GetData();
		TConstArrayView<float> InputData = Tensor.GetPreparedData<float>();

		check(Perms.Num() == TensorShape.Num());

		TArray<uint32> TransposedShape;
		TransposedShape.SetNumUninitialized(TensorShape.Num());
		for (int32 i = 0; i < TensorShape.Num(); ++i)
		{
			TransposedShape[i] = TensorShape[Perms[i]];
		}

		TArray<float> TransposedData;
		TransposedData.SetNumUninitialized(InputData.Num());
		TArray<uint32> TransposedPosition;
		TransposedPosition.SetNumUninitialized(TransposedShape.Num());

		Private::TensorIdxIterator it(Tensor.GetShape());
		const Private::TensorIdxIterator itTransposed(NNE::FTensorShape::Make(TransposedShape));

		do
		{
			TConstArrayView<uint32> Position = it.GetPositions();
			for (int32 i = 0; i < TensorShape.Num(); ++i)
			{
				TransposedPosition[i] = Position[Perms[i]];
			}

			float Value = InputData[it.GetIndex()];
			int TransposedIndex = itTransposed.GetIndexFromPosition(TransposedPosition);
			TransposedData[TransposedIndex] = Value;

		} while (it.Advance());

		OutputTensor.SetPreparedData<float>(TransposedData);

		return true;
	}

	bool TransposePreparedData(FTensor& Tensor, TConstArrayView<int32> Perms)
	{
		//Transpose in place
		return Apply(Tensor, Perms, Tensor);
	}
	
} // UE::NNERuntimeRDG::Private::CPUHelper::Transpose
