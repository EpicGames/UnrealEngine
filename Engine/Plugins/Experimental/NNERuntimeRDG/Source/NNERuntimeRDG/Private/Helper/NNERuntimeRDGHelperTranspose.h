// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

namespace UE::NNERuntimeRDG::Private::CPUHelper::Transpose
{
	bool NNERUNTIMERDG_API Apply(const FTensor& DataTensor, TConstArrayView<int32> Perms, FTensor& OutputTensor);
	bool NNERUNTIMERDG_API TransposePreparedData(FTensor& Tensor, TConstArrayView<int32> Perms);

} // UE::NNERuntimeRDG::Private::CPUHelper::Transpose