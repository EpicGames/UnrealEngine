// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGLogHelper.h"

namespace UE::NNERuntimeRDG::Private::LogHelper
{

FString GetTensorDataTypeName(ENNETensorDataType DataType)
{
	switch (DataType)
	{
		case ENNETensorDataType::None:		return TEXT("None");
		case ENNETensorDataType::Char:		return TEXT("Char");
		case ENNETensorDataType::Boolean:	return TEXT("Boolean");
		case ENNETensorDataType::Half:		return TEXT("Half");
		case ENNETensorDataType::Float:		return TEXT("Float");
		case ENNETensorDataType::Double:	return TEXT("Double");
		case ENNETensorDataType::Int8:		return TEXT("Int8");
		case ENNETensorDataType::Int16:		return TEXT("Int16");
		case ENNETensorDataType::Int32:		return TEXT("Int32");
		case ENNETensorDataType::Int64:		return TEXT("Int64");
		case ENNETensorDataType::UInt8:		return TEXT("UInt8");
		case ENNETensorDataType::UInt16:	return TEXT("UInt16");
		case ENNETensorDataType::UInt32:	return TEXT("UInt32");
		case ENNETensorDataType::UInt64:	return TEXT("UInt64");
		case ENNETensorDataType::Complex64:	return TEXT("Complex64");
		case ENNETensorDataType::Complex128:return TEXT("Complex128");
		case ENNETensorDataType::BFloat16:	return TEXT("BFloat16");
		default: return TEXT("Unknown!");
	}
}

} // UE::NNERuntimeRDG::Private::LogHelper

