// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::LogHelper
{
	FString GetTensorDataTypeName(ENNETensorDataType DataType);

} // UE::NNERuntimeRDG::Private::LogHelper

