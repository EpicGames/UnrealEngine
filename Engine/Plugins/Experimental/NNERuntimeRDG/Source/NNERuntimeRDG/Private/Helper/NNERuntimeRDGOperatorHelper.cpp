// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGOperatorHelper.h"

namespace UE::NNERuntimeRDG::Private::OperatorHelper
{
	template<typename Allocator>
	bool GetInt32ArrayFromConstTensorImpl(TArray<int32, Allocator>& Attr, const FTensorRef Tensor)
	{
		check(Tensor != nullptr);
		Attr.Reset();

		if (!Tensor->IsConstant())
		{
			return false;
		}

		if (Tensor->GetDataType() == ENNETensorDataType::Int64)
		{
			for (int64 Value64 : Tensor->GetPreparedData<int64>())
			{
				int64 ValueClamped = FMath::Clamp<int64>(Value64, MIN_int32, MAX_int32);
				Attr.Add((int32)ValueClamped);
			}

			return true;
		}
		else if (Tensor->GetDataType() == ENNETensorDataType::Int32)
		{
			Attr.Append(Tensor->GetPreparedData<int32>());
			return true;
		}

		return false;
	}
	
	bool GetInt32ArrayFromConstTensor(TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Attr, const FTensorRef Tensor)
	{
		return GetInt32ArrayFromConstTensorImpl(Attr, Tensor);
	}
	
	bool GetInt32ArrayFromConstTensor(TArray<int32, TInlineAllocator<2 * NNE::FTensorShape::MaxRank>>& Attr, const FTensorRef Tensor)
	{
		return GetInt32ArrayFromConstTensorImpl(Attr, Tensor);
	}

	bool GetInt32ArrayFromConstTensor(TArray<int32>& Attr, const FTensorRef Tensor)
	{
		return GetInt32ArrayFromConstTensorImpl(Attr, Tensor);
	}

} // UE::NNERuntimeRDG::Private::OperatorHelper

