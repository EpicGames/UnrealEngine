// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::OperatorHelper
{
	bool GetInt32ArrayFromConstTensor(TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Attr, const FTensorRef Tensor);
	bool GetInt32ArrayFromConstTensor(TArray<int32, TInlineAllocator<2 * NNE::FTensorShape::MaxRank>>& Attr, const FTensorRef Tensor);
	bool GetInt32ArrayFromConstTensor(TArray<int32>& Attr, const FTensorRef Tensor);

} // UE::NNERuntimeRDG::Private::OperatorHelper

