// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGTensorIdxIterator.h"

namespace UE::NNERuntimeRDG::Private
{
	TensorIdxIterator::TensorIdxIterator(const NNE::FTensorShape& InTensorShape) : TensorShape(InTensorShape)
	{
		CurrentPosition.Init(0, InTensorShape.Rank());
	}

	bool TensorIdxIterator::Advance()
	{
		for (int32 i = TensorShape.Rank() - 1; i >= 0; --i)
		{
			++CurrentPosition[i];
			if (CurrentPosition[i] < TensorShape.GetData()[i])
			{
				return true;
			}
			CurrentPosition[i] = 0;
		}
		return false;
	}

	TConstArrayView<uint32> TensorIdxIterator::GetPositions() const
	{
		return CurrentPosition;
	}

	int32 TensorIdxIterator::GetIndexToBroadcastedShape(const NNE::FTensorShape& InTensorShape) const
	{
		int32 Index = 0;
		int32 DimBaseOffset = 1;
		for (int32 r = TensorShape.Rank() - 1; r >= 0; --r)
		{
			if (r >= InTensorShape.Rank())
			{
				break;
			}
			Index += FMath::Min(CurrentPosition[r], InTensorShape.GetData()[r]-1) * DimBaseOffset;
			DimBaseOffset *= FMath::Min(TensorShape.GetData()[r], InTensorShape.GetData()[r]);
		}
		return Index;
	}

	int32 TensorIdxIterator::GetIndex() const
	{
		int32 Index = 0;
		int32 DimBaseOffset = 1;
		for (int32 r = TensorShape.Rank() - 1; r >= 0; --r)
		{
			Index += CurrentPosition[r] * DimBaseOffset;
			DimBaseOffset *= TensorShape.GetData()[r];
		}
		return Index;
	}

	int32 TensorIdxIterator::GetIndexFromPosition(TConstArrayView<uint32> Position) const
	{
		int32 Index = 0;
		int32 DimBaseOffset = 1;
		for (int32 r = TensorShape.Rank() - 1; r >= 0; --r)
		{
			check(Position[r] < TensorShape.GetData()[r]);
			Index += Position[r] * DimBaseOffset;
			DimBaseOffset *= TensorShape.GetData()[r];
		}
		return Index;
	}

} // UE::NNERuntimeRDG::Private
