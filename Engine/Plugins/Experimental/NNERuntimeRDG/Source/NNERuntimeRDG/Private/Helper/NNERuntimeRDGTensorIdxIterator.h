// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private
{
	class TensorIdxIterator
	{
		const NNE::FTensorShape& TensorShape;
		TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> CurrentPosition;

	public:
		TensorIdxIterator(const NNE::FTensorShape& InTensorShape);
		bool Advance();
		int32 GetIndexToBroadcastedShape(const NNE::FTensorShape& InTensorShape) const;
		int32 GetIndex() const;
		TConstArrayView<uint32> GetPositions() const;
		int32 GetIndexFromPosition(TConstArrayView<uint32> Position) const;
	};

} // UE::NNERuntimeRDG::Private
