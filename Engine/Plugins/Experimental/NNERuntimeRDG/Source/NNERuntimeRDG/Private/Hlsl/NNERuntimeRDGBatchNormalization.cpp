// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGBatchNormalization.h"

#include "NNEHlslShadersBatchNormalizationCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorBatchNormalization, TEXT("NNE.Operator.Hlsl.BatchNormalization"));

	/**
	 * BatchNormalization operator implementation
	 */
	class FBatchNormalization : public FOperatorHlsl
	{
	public:

		FBatchNormalization() {}
		virtual ~FBatchNormalization() = default;

		float Epsilon;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 5);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];

			OutputTensors[0]->SetShape(X.GetShape());

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 5);
			check(OutputTensorDescs.Num() >= 1);

			Epsilon = Attributes.GetValueOrDefault<float>(TEXT("epsilon"), 1e-5f);

			if (OutputTensorDescs.Num() > 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "BatchNormalization: Only supported in inference mode at the moment, supporting no more than one output.");
				return false;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 5);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(InputTensors[2] != nullptr);
			check(InputTensors[3] != nullptr);
			check(InputTensors[4] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& X = *InputTensors[0];
			const FTensorRDG& Scales = *InputTensors[1];
			const FTensorRDG& Bias = *InputTensors[2];
			const FTensorRDG& Mean = *InputTensors[3];
			const FTensorRDG& Var = *InputTensors[4];
			const FTensorRDG& Output = *OutputTensors[0];

			FRDGBufferSRVRef XSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(X.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferSRVRef ScalesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Scales.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferSRVRef BiasSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferSRVRef MeanSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Mean.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferSRVRef VarSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Var.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FBatchNormalizationConstants::NUM_GROUP_THREADS);
			int32 DimC = 1;
			int32 SpatialVolume = 1;
			if (X.GetShape().Rank() >= 2)
			{
				DimC = X.GetShape().GetData()[1];
			}
			for (int32 i = X.GetShape().Rank() - 1; i > 1; --i)
			{
				SpatialVolume *= X.GetShape().GetData()[i];
			}

			// Set parameters
			TBatchNormalizationCS::FParameters* Params = GraphBuilder.AllocParameters<TBatchNormalizationCS::FParameters>();
			Params->X = XSRV;
			Params->Scales = ScalesSRV;
			Params->Bias = BiasSRV;
			Params->Mean = MeanSRV;
			Params->Var = VarSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->DimC = DimC;
			Params->SpatialVolume = SpatialVolume;
			Params->ThreadCountX = ThreadGroupCount.X * FBatchNormalizationConstants::NUM_GROUP_THREADS;
			Params->Epsilon = Epsilon;

			TBatchNormalizationCS::FPermutationDomain PermutationVector;
			TShaderMapRef<TBatchNormalizationCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorBatchNormalization, "NNE.Operator.Hlsl.BatchNormalization");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.BatchNormalization.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateBatchNormalizationOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputBatchNormalizations)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("epsilon"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("momentum"), ENNERuntimeRDGDataAttributeDataType::Float); //Will be ignored, only useful in training mode.
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateBatchNormalizationOperator()
	{
		return new FBatchNormalization();
	}

	bool RegisterBatchNormalizationOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("BatchNormalization"), TEXT("Onnx")}, 9}, CreateBatchNormalizationOperator, ValidateBatchNormalizationOperator);
		// Next version: 14
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
