// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGCast.h"

#include "Helper/NNERuntimeRDGHelperCast.h"
#include "Helper/NNERuntimeRDGLogHelper.h"
#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNEHlslShadersCastCS.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersTypeHelper.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorCast, TEXT("NNE.Operator.Hlsl.Cast"));
	/**
	 * Cast operator implementation
	 */
	class FCast : public FOperatorHlsl
	{
	public:

		FCast() {}
		virtual ~FCast() = default;

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			OutputTensors[0]->SetShape(InputTensors[0]->GetShape());

			const FTensor& Input = *InputTensors[0];
			FTensor& Output = *OutputTensors[0];

			CPUHelper::Cast::Apply(Input, Output);

			bool Has64BitDataType = Input.GetDataType() == ENNETensorDataType::Int64 || Output.GetDataType() == ENNETensorDataType::UInt64;

			if (!Output.HasPreparedData() && Has64BitDataType)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Cast: Output could not be computed as a constant tensor, however Cast doesn't support dynamic 64 bit tensor types.");
				return -1;
			}

			return 0;
		}

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			ENNETensorDataType ToFromAttribute = (ENNETensorDataType)Attributes.GetValue<int32>(TEXT("to"));
			ENNETensorDataType ToFromTensor = OutputTensorDescs[0].GetDataType();

			if (ToFromAttribute != ToFromTensor)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Cast: Should output a tensor of type %d but was of type %d.", int(ToFromAttribute), int(ToFromTensor));
				return false;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;
			
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), TensorDataTypeToPixelFormat(Input.GetDataType())));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), TensorDataTypeToPixelFormat(Output.GetDataType())));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FCastConstants::NUM_GROUP_THREADS);

			// Set parameters
			TCastCS::FParameters* Params = GraphBuilder.AllocParameters<TCastCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FCastConstants::NUM_GROUP_THREADS;

			TCastCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TCastCS::FInputType>(TensorToShaderDataType(Input.GetDataType()));
			PermutationVector.Set<TCastCS::FOutputType>(TensorToShaderDataType(Output.GetDataType()));

			TShaderMapRef<TCastCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorCast, "NNE.Operator.Hlsl.Cast");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Cast.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateCastOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddRequired(TEXT("to"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		if (bIsValid)
		{
			ENNETensorDataType To = (ENNETensorDataType)AttributeMap.GetValue<int32>(TEXT("to"));
			switch (To)
			{
				case ENNETensorDataType::Half:
				case ENNETensorDataType::Float:
				case ENNETensorDataType::Int32:
				case ENNETensorDataType::Int64:
					break;
				default:
					FString TargetType = LogHelper::GetTensorDataTypeName(To);
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Cast: Target tensor data type %ls not supported.", *TargetType);
					bIsValid = false;
			}
		}
		
		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateCastOperator()
	{
		return new FCast();
	}

	bool RegisterCastOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Cast"), TEXT("Onnx")}, 6}, CreateCastOperator, ValidateCastOperator);
		Registry.OpAdd({{TEXT("Cast"), TEXT("Onnx")}, 9}, CreateCastOperator, ValidateCastOperator);
		Registry.OpAdd({{TEXT("Cast"), TEXT("Onnx")}, 13}, CreateCastOperator, ValidateCastOperator);
		// Next version: 19
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
