// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGConcat.h"

#include "Helper/NNERuntimeRDGHelperConcat.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	namespace ConcatHelper
	{
		uint64 GetNumElemLeftOfAxisForShape(const NNE::FTensorShape& Shape, int32 Axis)
		{
			TConstArrayView<uint32> Data = Shape.GetData();

			check(Axis >= 0);
			check(Axis < Data.Num());

			uint64 Result = 1;

			for (int32 Idx = 0; Idx < Axis; ++Idx)
			{
				Result *= Data[Idx];
			}
			return Result;
		}

		uint64 GetNumElemRightOfAxisIncludedForShape(const NNE::FTensorShape& Shape, int32 Axis)
		{
			const uint64 NumElemLeftOfAxis = GetNumElemLeftOfAxisForShape(Shape, Axis);

			check(NumElemLeftOfAxis != 0);
			return Shape.Volume() / NumElemLeftOfAxis;
		}
	}

	DECLARE_GPU_STAT_NAMED(FNNEOperatorConcat, TEXT("NNE.Operator.Hlsl.Concat"));

	/**
	 * Concat operator implementation
	 */
	class FConcat : public FOperatorHlsl
	{
	public:

		FConcat() {}
		virtual ~FConcat() = default;

		int32 Axis;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 1);
			check(OutputTensors.Num() == 1);

			TArray<uint32> OutputShapeData(InputTensors[0]->GetShape().GetData());
			
			for (int32 i = 1; i < InputTensors.Num(); ++i)
			{
				OutputShapeData[Axis] += InputTensors[i]->GetShape().GetData()[Axis];
				
				for (int32 r = 0; r < OutputShapeData.Num(); ++r)
				{
					if (r != Axis && (OutputShapeData[r] != InputTensors[i]->GetShape().GetData()[r]))
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Concat: All input tensors should have the same shape except on the concatenation axis");
						return false;
					}
				}
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			
			OutputTensors[0]->SetShape(OutputShape);

			CPUHelper::Concat::Apply(InputTensors, *OutputTensors[0], Axis);
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() >= 1);
			check(OutputTensorDescs.Num() == 1);

			Axis = Attributes.GetValue<int32>(TEXT("axis"));
			Axis = Axis >= 0 ? Axis : InputTensorDescs[0].GetShape().Rank() + Axis;
			
			int32 InputsRank = InputTensorDescs[0].GetShape().Rank();
			
			for (int32 i = 1; i < InputTensorDescs.Num(); ++i)
			{
				if (InputsRank != InputTensorDescs[i].GetShape().Rank())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Concat: All input tensors should have the same rank");
					return false;
				}
			}
			
			if (Axis < -InputsRank || Axis >(InputsRank - 1))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Concat: Axis should be in range [-r,r-1] however it is %d while inputs have rank %d.", Axis, InputsRank);
				return false;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 1);
			check(OutputTensors.Num() == 1);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Output = *OutputTensors[0];
			const uint64 NumElemLeftOfAxis = ConcatHelper::GetNumElemLeftOfAxisForShape(Output.GetShape(), Axis);
			uint64 OutputOffset = 0;

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConcat, "NNE.Operator.Hlsl.Concat");

			for (uint64 IndexShapeLeftOfAxis = 0; IndexShapeLeftOfAxis < NumElemLeftOfAxis; ++IndexShapeLeftOfAxis)
			{
				for (FTensorRDGRef InputTensorRef : InputTensors)
				{
					check(InputTensorRef != nullptr);

					const FTensorRDG& InputTensor = *InputTensorRef;
					const uint64 NumElemToCopy = ConcatHelper::GetNumElemRightOfAxisIncludedForShape(InputTensor.GetShape(), Axis);
					const uint64 FirstElementToCopy = IndexShapeLeftOfAxis * NumElemToCopy;
					const uint64 NumBytesToCopy = NumElemToCopy * InputTensor.GetElementByteSize();
					const uint64 FirstByteToCopy = FirstElementToCopy * InputTensor.GetElementByteSize();

					AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), OutputOffset, InputTensor.GetBuffer(), FirstByteToCopy, NumBytesToCopy);

					OutputOffset += NumBytesToCopy;
				}
			}

			ensureAlwaysMsgf(OutputOffset == Output.GetDataSize(), TEXT("NNE.Operator.Hlsl.Concat: All of the output buffer should have been written down"));
		}
	};

	bool ValidateConcatOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputConcats)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddRequired(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		if (InputTypes.Num() == 0)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Concat: Operator requires at least 1 input");
			bIsValid = false;
		}
		for (int32 i = 0; i < InputTypes.Num(); ++i)
		{
			if (InputTypes[i] != ENNETensorDataType::Float && InputTypes[i] != ENNETensorDataType::Half)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Concat: Operator input '%d' of type '%d' is not supported, should be float or half at the moment.", i, int(InputTypes[i]));
				bIsValid = false;
			}
			if (InputTypes[i] != InputTypes[0])
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Concat: Operator input '%d' of type '%d' does not match type '%d' of the first input.", i, int(InputTypes[i]), int(InputTypes[0]));
				bIsValid = false;
			}
		}

		return bIsValid;
	}

	FOperatorHlsl* CreateConcatOperator()
	{
		return new FConcat();
	}

	bool RegisterConcatOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Concat"), TEXT("Onnx")}, 4}, CreateConcatOperator, ValidateConcatOperator);
		Registry.OpAdd({{TEXT("Concat"), TEXT("Onnx")}, 11}, CreateConcatOperator, ValidateConcatOperator);
		Registry.OpAdd({{TEXT("Concat"), TEXT("Onnx")}, 13}, CreateConcatOperator, ValidateConcatOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
