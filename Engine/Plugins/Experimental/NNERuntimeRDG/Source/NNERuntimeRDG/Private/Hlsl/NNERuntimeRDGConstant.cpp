// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGConstant.h"

#include "Helper/NNERuntimeRDGLogHelper.h"
#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeTensor.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	/**
	 * Constant operator implementation
	 */
	class FConstant : public FOperatorHlsl
	{
	public:

		FConstant() {}
		virtual ~FConstant() = default;

	private:

		FNNERuntimeRDGDataAttributeValue Attribute;


	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			using NNE::FTensorShape;
			using NNERuntimeRDGData::Internal::FAttributeTensor;

			check(InputTensors.Num() == 0);
			check(OutputTensors.Num() == 1);

			FTensor& Output = *OutputTensors[0];

			switch (Attribute.GetType())
			{
				case ENNERuntimeRDGDataAttributeDataType::Float:
				{
					if (Output.GetDataType() != ENNETensorDataType::Float)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Constant: Output data type %ls does not match constant type of float", 
							   					*LogHelper::GetTensorDataTypeName(Output.GetDataType()));
						return -1;
					}
					const float Value = Attribute.GetValue<float>();
					const TArray<uint32, TInlineAllocator<FTensorShape::MaxRank>> Shape;
					const TConstArrayView<float> Data = MakeArrayView(&Value, 1);
					Output.SetShape(FTensorShape::Make(Shape));
					Output.SetPreparedData(Data);
				} break;
				case ENNERuntimeRDGDataAttributeDataType::FloatArray:
				{
					if (Output.GetDataType() != ENNETensorDataType::Float)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Constant: Output data type %ls does not match constant type of float", 
							   					*LogHelper::GetTensorDataTypeName(Output.GetDataType()));
						return -1;
					}
					const TArray<float> Values = Attribute.GetValue<TArray<float>>();
					TArray<uint32, TInlineAllocator<FTensorShape::MaxRank>> Shape;
					Shape.Add(Values.Num());
					Output.SetShape(FTensorShape::Make(Shape));
					Output.SetPreparedData(MakeArrayView(Values));
				} break;
				case ENNERuntimeRDGDataAttributeDataType::Tensor:
				{
					const FAttributeTensor AttributeTensor = Attribute.GetValue<FAttributeTensor>();
					if (Output.GetDataType() != AttributeTensor.GetDataType())
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Constant: Output data type %ls does not match constant tensor data type %ls", 
							   					*LogHelper::GetTensorDataTypeName(Output.GetDataType()), 
												*LogHelper::GetTensorDataTypeName(AttributeTensor.GetDataType()));
						return -1;
					}
					Output.SetShape(AttributeTensor.GetShape());
					Output.SetPreparedData(AttributeTensor.GetData());
				} break;
				default:
					check(false);
					break;
			}
			check(OutputTensors[0]->IsConstant());
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 0);
			check(OutputTensorDescs.Num() == 1);
			check(Attributes.Num() == 1);

			Attribute = Attributes.GetAttributeValue(0);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			checkf(false, TEXT("Dispatch should never be called, since we have a constant output"));
		}
	};

	bool ValidateConstantOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("value"), ENNERuntimeRDGDataAttributeDataType::Tensor);
		AttributeValidator.AddOptional(TEXT("value_float"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("value_floats"), ENNERuntimeRDGDataAttributeDataType::FloatArray);
		if (!AttributeValidator.Validate(AttributeMap))
		{
			return false;
		}

		if (AttributeMap.Num() != 1)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Constant: Operator requires exacly one attribute, but '%d' attributes found.", AttributeMap.Num());
			return false;
		}

		FInputValidator InputValidator;
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateConstantOperator()
	{
		return new FConstant();
	}

	bool RegisterConstantOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		#define OP(Version) \
		Registry.OpAdd({{TEXT("Constant"), TEXT("Onnx")}, Version}, CreateConstantOperator, ValidateConstantOperator);

		OP(9)
		OP(11)
		OP(12)
		OP(13)
		OP(19)
		OP(21)

		#undef OP
		return true;
	}

} // UE::NNERuntimeRDG::Private::Hlsl
