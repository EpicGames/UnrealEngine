// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGConv.h"

#include "Algo/AnyOf.h"
#include "HAL/IConsoleManager.h"
#include "Helper/NNERuntimeRDGHelperTranspose.h"
#include "NNEHlslShadersConvCS.h"
#include "NNEHlslShadersConvMatmulCS.h"
#include "NNEHlslShadersConvWinogradInputCS.h"
#include "NNEHlslShadersConvWinogradMMMCS.h"
#include "NNEHlslShadersConvWinogradOutputCS.h"
#include "NNEHlslShadersConvWinogradWeightsCS.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersTypeHelper.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvDefault, TEXT("NNE.Operator.Hlsl.Conv.Default"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvMatmul, TEXT("NNE.Operator.Hlsl.Conv.Matmul"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvWinogradInput, TEXT("NNE.Operator.Hlsl.Conv.Winograd.Input"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvWinogradMMM, TEXT("NNE.Operator.Hlsl.Conv.Winograd.MMM"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvWinogradOutput, TEXT("NNE.Operator.Hlsl.Conv.Winograd.Output"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvWinogradWeights, TEXT("NNE.Operator.Hlsl.Conv.Winograd.Weights"));

	using EConvAutoPad = UE::NNEHlslShaders::Internal::EConvAutoPad;
	using EConvGroupSize = UE::NNEHlslShaders::Internal::EConvGroupSize;


	TAutoConsoleVariable<int32> CVarWinogradPrecision(
		TEXT("nne.hlsl.WinogradPrecision"),
		0,
		TEXT("Selects the Precision of the Winograd Convolution implementation.\n")
		TEXT(" 0: Allow Float16 (fast, low precision) (default)\n")
		TEXT(" 1: Disable Float16 (medium speed, medium precision)\n")
		TEXT(" 2: Disable Winograd (slow)")
	);

	/**
	 * Convolution operator implementation
	 */
	class FConv : public FOperatorHlsl
	{
	public:

		// The first values have to correspond to the values in CVarWinogradPrecision
		// so that it is possible to convert from the CVar value to this enum.
		enum class EWinogradPrecision
		{
			FP16 = 0,
			FP32,
			DISABLED,

			UNDEFINED,

			MAX
		};

		static FOperatorHlsl* Create()
		{
			return new FConv();
		}

		virtual ~FConv() = default;

	private:

		FConv() {}

		int32 NumDimensions = 0;

		EConvAutoPad AutoPad = EConvAutoPad::NOTSET;
		TArray<int32> Dilations;
		bool bHasDilation;
		int32 Group = 1;
		TArray<int32> Pads;
		TArray<int32> Strides;
		bool bAreWeightsTransposed = false;
		EPixelFormat BufferPixelFormat;
		EConvGroupSize GroupSize = EConvGroupSize::MAX;
		EWinogradPrecision WinogradTestPrecision;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);

			const NNE::FTensorShape& Input = InputTensors[0]->GetShape();
			const NNE::FTensorShape& Weights = InputTensors[1]->GetShape();

			GroupSize = FConvCS::GetBiggestCompatibleGroupSize(Weights.GetData(), Dilations, Strides);
			if(GroupSize == EConvGroupSize::MAX)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: Kernel size, strides, dilations combination is not supported. Kernel tensor: %ls.", *InputTensors[1]->GetName());
				return -1;
			}

			const TArray<int32> OutputShapeData = NNEHlslShaders::Internal::FConvCS::GetOutputShape(Input.GetData(), Weights.GetData(), AutoPad, Dilations, Strides, Pads);
			const NNE::FSymbolicTensorShape OutputShape = NNE::FSymbolicTensorShape::Make(OutputShapeData);
			
			if (!OutputShape.IsConcrete())
			{
				return -1;
			}
			OutputTensors[0]->SetShape(NNE::FTensorShape::MakeFromSymbolic(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
            using namespace UE::NNEHlslShaders::Internal;

			check(InputTensorDescs.Num() >= 2 && InputTensorDescs.Num() <= 3);
			check(OutputTensorDescs.Num() == 1);

            const NNE::FTensorDesc& Input = InputTensorDescs[0];
			const NNE::FTensorDesc& Weights = InputTensorDescs[1];
			const NNE::FTensorDesc& Output = OutputTensorDescs[0];
			
			if (Input.GetShape().Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: First input should be at least of rank 2");
				return false;
			}
			if (Weights.GetShape().Rank() != Input.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: First and second inputs should be of same ranks");
				return false;
			}
			if (Output.GetShape().Rank() != Input.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: First and output should be of same ranks");
				return false;
			}

			NumDimensions = Input.GetShape().Rank() - 2;

			TArray<int32> DilationsOrStridesDefault;
			DilationsOrStridesDefault.Init(1, NumDimensions);

			FConvCS::LexFromString(AutoPad, *Attributes.GetValueOrDefault<FString>(TEXT("auto_pad"), TEXT("NOTSET")));
			Dilations = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("dilations"), DilationsOrStridesDefault);
			bHasDilation = Algo::AnyOf(Dilations, [](auto Dim) {return Dim != 1u; });
			Group = Attributes.GetValueOrDefault<int32>(TEXT("group"), 1);
			if (AutoPad == EConvAutoPad::NOTSET)
			{
				TArray<int32> PadsDefault;
				PadsDefault.Init(0, 2 * NumDimensions);

				Pads = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("pads"), PadsDefault);
			}
			Strides = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("strides"), DilationsOrStridesDefault);
			if (Strides.Num() != NumDimensions)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: Strides count doesn't match the number of spatial dimensions");
				return false;
			}
			BufferPixelFormat = TensorDataTypeToPixelFormat(Input.GetDataType());

			int32 WinogradTestPrecisionValue = Attributes.GetValueOrDefault<int32>(TEXT("__UE__WinogradPrecision"), static_cast<int32>(EWinogradPrecision::UNDEFINED));
			WinogradTestPrecisionValue = FMath::Clamp(WinogradTestPrecisionValue, 0, static_cast<int32>(EWinogradPrecision::MAX) - 1);
			WinogradTestPrecision = static_cast<EWinogradPrecision>(WinogradTestPrecisionValue);

			return true;
		}

		void DispatchConvDefault (FRDGBuilder& GraphBuilder, const FTensorRDG& Input, const FTensorRDG& Weights, const FTensorRDG* Bias, const FTensorRDG& Output) const
		{
			using namespace UE::NNEHlslShaders::Internal;

			constexpr EConvAlgorithm Algorithm = EConvAlgorithm::SharedMemory;
			
			bool HasBias = Bias != nullptr;
			TArray<int32> OutputShape = FConvCS::GetOutputShape(Input.GetShape().GetData(), Weights.GetShape().GetData(), AutoPad, Dilations, Strides, Pads);

			// Set parameters
			FConvCS::FParameters* Params = GraphBuilder.AllocParameters<FConvCS::FParameters>();
			FConvCS::FillInParameters(GroupSize, Input.GetShape().GetData(), Weights.GetShape().GetData(), HasBias, AutoPad, Group, Dilations,Strides, Pads, *Params);
			Params->X = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), BufferPixelFormat));
			Params->W = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Weights.GetBuffer(), BufferPixelFormat));
			if (HasBias) {
				Params->B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias->GetBuffer(), BufferPixelFormat));
			}
			Params->Y = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), BufferPixelFormat));

			FConvCS::FPermutationDomain PermutationVector;

			PermutationVector.Set<FConvCS::FConvAlgorithm>(Algorithm);
			PermutationVector.Set<FConvCS::FConvAreWeightsTransposed>(bAreWeightsTransposed);
			PermutationVector.Set<FConvCS::FConvGroupSize>(GroupSize);
			PermutationVector.Set<FConvCS::FConvNumDimensions>(NumDimensions);
			PermutationVector.Set<FConvCS::FConvNumReadsPerThread>(FConvCS::GetNumReadsPerThread(GroupSize, Weights.GetShape().GetData(), Dilations, Strides));
			PermutationVector.Set<FConvCS::FConvHasB>(HasBias);
			TShaderMapRef<FConvCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvDefault, "NNE.Operator.Hlsl.Conv.Default");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Default.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				FConvCS::GetGroupCount(OutputShape, FConvCS::GetGroupShape(GroupSize, NumDimensions)));
		}

		bool DispatchConvMatmul(FRDGBuilder& GraphBuilder, const FTensorRDG& Input, const FTensorRDG& Weights, const FTensorRDG* Bias, const FTensorRDG& Output) const
		{
			using namespace UE::NNEHlslShaders::Internal;

			if (Group != 1)
				return false;
			if (Input.GetShape().Rank() != 4)
				return false;
			if (Output.GetShape().Rank() != 4)
				return false;
			if (Weights.GetShape().Rank() != 4)
				return false;
			if (bHasDilation)
				return false;

			int Ni = Input.GetShape().GetData()[0];
			int Ci = Input.GetShape().GetData()[1];
			int Hi = Input.GetShape().GetData()[2];
			int Wi = Input.GetShape().GetData()[3];

			check(Ni == Output.GetShape().GetData()[0]);
			int Cw = Output.GetShape().GetData()[1];
			int Ho = Output.GetShape().GetData()[2];
			int Wo = Output.GetShape().GetData()[3];

			check(Cw == Weights.GetShape().GetData()[0]);
			check(Ci == Weights.GetShape().GetData()[1]);
			int Hw = Weights.GetShape().GetData()[2];
			int Ww = Weights.GetShape().GetData()[3];

			if (Wo % 32 != 0) // Idea : support this by launching more threads and discard some results so a threadgroup still operator on only one value for H.
				return false;

			TArray<int32> Padding = FConvCS::GetPadding(Input.GetShape().GetData(), Weights.GetShape().GetData(), AutoPad, Dilations, Strides, Pads);
			bool bHasBias = Bias != nullptr;

			// Set parameters
			FConvMatmulCS::FParameters* Params = GraphBuilder.AllocParameters<FConvMatmulCS::FParameters>();
			Params->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), BufferPixelFormat));
			Params->Weight= GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Weights.GetBuffer(), BufferPixelFormat));
			if (bHasBias) {
				Params->Bias = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias->GetBuffer(), BufferPixelFormat));
			}
			Params->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), BufferPixelFormat));
			Params->Ci = Ci;
			Params->Hi = Hi;
			Params->Wi = Wi;
			Params->Cw = Cw;
			Params->Hw = Hw;
			Params->Ww = Ww;
			Params->Ho = Ho;
			Params->Wo = Wo;
			Params->StrideH = Strides[0];
			Params->StrideW = Strides[1];
			Params->PadTop = Padding[0];
			Params->PadLeft = Padding[1];

			FConvMatmulCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FConvMatmulCS::FConvMatmulAreWeightsTransposed>(bAreWeightsTransposed);
			PermutationVector.Set<FConvMatmulCS::FConvMatmulHasBias>(bHasBias);
			TShaderMapRef<FConvMatmulCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvMatmul, "NNE.Operator.Hlsl.Conv.Matmul");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Matmul.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				FConvMatmulCS::GetGroupCount(Output.GetShape().GetData()));

			return true;
		}

		// This algorithm is inspired by this paper: https://github.com/xuqiantong/CUDA-Winograd/blob/master/report.pdf
		bool DispatchConvWinograd(FRDGBuilder& GraphBuilder, const FTensorRDG& Input, const FTensorRDG& Weights, const FTensorRDG* Bias, const FTensorRDG& Output) const
		{
			using namespace UE::NNEHlslShaders::Internal;

			if (Group != 1)
				return false;
			if (Input.GetShape().Rank() != 4)
				return false;
			if (Output.GetShape().Rank() != 4)
				return false;
			if (Weights.GetShape().Rank() != 4)
				return false;

			check(Strides.Num() == 2);
			if (Strides[0] != 1 || Strides[1] != 1)
				return false;
			
			if (bHasDilation)
				return false;

			TArray<int32> Padding = FConvCS::GetPadding(Input.GetShape().GetData(), Weights.GetShape().GetData(), AutoPad, Dilations, Strides, Pads);

			check(Padding.Num() == 4);
			if (Padding[0] != 1 || Padding[1] != 1 || Padding[2] != 1 || Padding[3] != 1)
				return false;

			const int Hw = Weights.GetShape().GetData()[2];
			const int Ww = Weights.GetShape().GetData()[3];
			if (Hw != 3 || Ww != 3)
				return false;

			check(!bAreWeightsTransposed);

			const int Ni = Input.GetShape().GetData()[0];
			const int Ci = Input.GetShape().GetData()[1];
			const int Hi = Input.GetShape().GetData()[2];
			const int Wi = Input.GetShape().GetData()[3];

			check(Ni == Output.GetShape().GetData()[0]);
			const int Cw = Output.GetShape().GetData()[1];
			check(Hi == Output.GetShape().GetData()[2]);
			check(Wi == Output.GetShape().GetData()[3]);

			check(Cw == Weights.GetShape().GetData()[0]);
			check(Ci == Weights.GetShape().GetData()[1]);

			EWinogradPrecision Precision = WinogradTestPrecision;
			if (Precision == EWinogradPrecision::UNDEFINED)
			{
				int32 CVarValue = CVarWinogradPrecision.GetValueOnRenderThread();
				CVarValue = FMath::Clamp(CVarValue, 0, static_cast<int32>(EWinogradPrecision::UNDEFINED) - 1);
				Precision = static_cast<EWinogradPrecision>(CVarValue);
			}

			if (Precision == EWinogradPrecision::DISABLED)
				return false;
			
			check(Input.GetDataType() == ENNETensorDataType::Float || Input.GetDataType() == ENNETensorDataType::Half);
			const ENNETensorDataType TensorDataType = Precision == EWinogradPrecision::FP16 ? Input.GetDataType() : ENNETensorDataType::Float;


			const int BlockCountH = FMath::DivideAndRoundUp(Hi, 4);
			const int BlockCountW = FMath::DivideAndRoundUp(Wi, 4);

			//Ensures that the MMM shader gets called with an even number of elements in the M dimension
			const int BlockCountWExtended = ((BlockCountH * BlockCountW) % 2) != 0 ? RoundUp<2>(BlockCountW) : BlockCountW;
			const int CwExtended = RoundUp<2>(Cw);

			const int TransformedWeightSize = 36 * CwExtended * Ci;
			const int TransformedInputSize = Ni * 36 * (BlockCountH * BlockCountWExtended) * Ci;
			const int TransformedOutputSize = Ni * 36 * (BlockCountH * BlockCountWExtended) * CwExtended;

			const ENNEShaderDataType ShaderDataType = TensorToShaderDataType(TensorDataType);
			const EPixelFormat IntermediateBufferPixelFormat = TensorDataTypeToPixelFormat(TensorDataType);
			const EPixelFormat IntermediateBufferPixelFormat2 = TensorDataType == ENNETensorDataType::Float ? PF_G32R32F : PF_G16R16F;
			const uint32 IntermediateElementByteSize = UE::NNE::GetTensorDataTypeSizeInBytes(TensorDataType);

			const FRDGBufferDesc TransformedWeightsBufferDesc = FRDGBufferDesc::CreateBufferDesc(IntermediateElementByteSize, TransformedWeightSize);
			const FRDGBufferDesc TransformedInputBufferDesc = FRDGBufferDesc::CreateBufferDesc(IntermediateElementByteSize, TransformedInputSize);
			const FRDGBufferDesc TransformedOutputBufferDesc = FRDGBufferDesc::CreateBufferDesc(IntermediateElementByteSize, TransformedOutputSize);

			const FRDGBufferRef TransformedWeights = GraphBuilder.CreateBuffer(TransformedWeightsBufferDesc, TEXT("NNE.Tensor.ConvWinograd.TransformedWeights"), ERDGBufferFlags::None);
			const FRDGBufferRef TransformedInput = GraphBuilder.CreateBuffer(TransformedInputBufferDesc, TEXT("NNE.Tensor.ConvWinograd.TransformedInput"), ERDGBufferFlags::None);
			const FRDGBufferRef TransformedOutput = GraphBuilder.CreateBuffer(TransformedOutputBufferDesc, TEXT("NNE.Tensor.ConvWinograd.TransformedOutput"), ERDGBufferFlags::None);


			// Dispatch Weight transformation
			{
				FConvWinogradWeightsCS::FParameters* Params = GraphBuilder.AllocParameters<FConvWinogradWeightsCS::FParameters>();
				Params->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Weights.GetBuffer(), BufferPixelFormat));
				Params->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(TransformedWeights, IntermediateBufferPixelFormat));
				Params->Ci = Ci;
				Params->Cw = Cw;
				Params->CwInputStride = Ci;
				Params->MatrixOutputStride = Ci * CwExtended;
				Params->CiOutputStride = CwExtended;

				TShaderMapRef<FConvWinogradWeightsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvWinogradWeights, "NNE.Operator.Hlsl.Conv.Winograd.Weights");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Winograd.Weights.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					FIntVector(FMath::DivideAndRoundUp(Ci, FConvWinogradWeightsConstants::THREADGROUP_SIZE_X), CwExtended, 1));
			}
			// Dispatch Input transformation
			{
				FConvWinogradInputCS::FParameters* Params = GraphBuilder.AllocParameters<FConvWinogradInputCS::FParameters>();
				Params->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), BufferPixelFormat));
				Params->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(TransformedInput, IntermediateBufferPixelFormat));
				Params->C = Ci;
				Params->H = Hi;
				Params->W = Wi;
				Params->WBlockCount = BlockCountW;
				Params->CInputStride = Hi * Wi;
				Params->HInputStride = Wi;
				Params->NiOutputStride = 36 * Ci * BlockCountH * BlockCountWExtended;
				Params->MatrixOutputStride = Ci * BlockCountH * BlockCountWExtended;
				Params->COutputStride = BlockCountH * BlockCountWExtended;
				Params->HOutputStride = BlockCountWExtended;

				TShaderMapRef<FConvWinogradInputCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvWinogradInput, "NNE.Operator.Hlsl.Conv.Winograd.Input");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Winograd.Input.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					FIntVector(FMath::DivideAndRoundUp(BlockCountWExtended, FConvWinogradInputConstants::THREADGROUP_SIZE_X), BlockCountH, Ci * Ni));
			}
			// Dispatch MMM
			{
				const int VectorSize = 2;
				const int M = (BlockCountH * BlockCountWExtended);
				const int N = CwExtended;
				const int K = Ci;
				FConvWinogradMMMCS::FParameters* Params = GraphBuilder.AllocParameters<FConvWinogradMMMCS::FParameters>();
				Params->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TransformedInput, IntermediateBufferPixelFormat2));
				Params->Weight = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TransformedWeights, IntermediateBufferPixelFormat2));
				Params->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(TransformedOutput, IntermediateBufferPixelFormat2));
				Params->M = M / VectorSize;
				Params->N = N / VectorSize;
				Params->K = K;
				Params->MatrixInputStride = K * M / VectorSize;
				Params->KInputStride = M / VectorSize;
				Params->MatrixWeightStride = K * N / VectorSize;
				Params->KWeightStride = N / VectorSize;
				Params->MatrixOutputStride = N * M / VectorSize;
				Params->NOutputStride = M / VectorSize;

				const int BlockSizeN = FConvWinogradMMMCS::GetOptimalBlockSizeN(M, K, N);
				FConvWinogradMMMCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FConvWinogradMMMCS::FDataType>(ShaderDataType);
				PermutationVector.Set<FConvWinogradMMMCS::FBlockSizeN>(BlockSizeN);
				TShaderMapRef<FConvWinogradMMMCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvWinogradMMM, "NNE.Operator.Hlsl.Conv.Winograd.MMM");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Winograd.MMM.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					FIntVector(FMath::DivideAndRoundUp(M, 64), FMath::DivideAndRoundUp(N, BlockSizeN), 36 * Ni));
			}
			// Dispatch Output transformation
			{
				const int C = Cw;
				const int H = Hi;
				const int W = Wi;
				const bool bHasBias = Bias != nullptr;
				FConvWinogradOutputCS::FParameters* Params = GraphBuilder.AllocParameters<FConvWinogradOutputCS::FParameters>();
				Params->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TransformedOutput, IntermediateBufferPixelFormat));
				if (bHasBias)
				{
					Params->Bias = GraphBuilder.CreateSRV(Bias->GetBuffer(), BufferPixelFormat);
				}
				Params->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), BufferPixelFormat));
				Params->C = C;
				Params->H = H;
				Params->W = W;
				Params->WBlockCount = BlockCountW;
				Params->NiInputStride = 36 * CwExtended * BlockCountH * BlockCountWExtended;
				Params->MatrixInputStride = CwExtended * BlockCountH * BlockCountWExtended;
				Params->CInputStride = BlockCountH * BlockCountWExtended;
				Params->HInputStride = BlockCountWExtended;
				Params->COutputStride = H * W;
				Params->HOutputStride = W;

				FConvWinogradOutputCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FConvWinogradOutputCS::FHasBias>(bHasBias);
				PermutationVector.Set<FConvWinogradOutputCS::FDataType>(ShaderDataType);
				TShaderMapRef<FConvWinogradOutputCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvWinogradOutput, "NNE.Operator.Hlsl.Conv.Winograd.Output");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.Conv.Winograd.Output.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					FIntVector(FMath::DivideAndRoundUp(BlockCountW, FConvWinogradOutputConstants::THREADGROUP_SIZE_X), BlockCountH, Cw * Ni));
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Weights = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];
			const FTensorRDG* Bias = nullptr;

			if (InputTensors.Num() == 3) {
				check(InputTensors[2] != nullptr);
				Bias = InputTensors[2];
			}

			check(Input.GetShape().Rank() > 2);
			check(Weights.GetShape().Rank() == Input.GetShape().Rank());
			check(Output.GetShape().Rank() == Input.GetShape().Rank());
			check(NumDimensions == (Input.GetShape().Rank() - 2));

#if !PLATFORM_MAC
			if (DispatchConvWinograd(GraphBuilder, Input, Weights, Bias, Output))
			{
				return;
			}
#endif // !PLATFORM_MAC
			if (DispatchConvMatmul(GraphBuilder, Input, Weights, Bias, Output))
			{
				return;
			}
			DispatchConvDefault(GraphBuilder, Input, Weights, Bias, Output);
		}

		virtual void OptimizeInputsWeights(TArrayView<FTensorRDGRef> InputWeights) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputWeights.Num() >= 2);
			FTensorRDGRef Weights = InputWeights[1];
			if (Weights == nullptr)
				return;

			// Heuristics : only matmul implementation benefits from transposed weights
			if (Weights->GetShape().Rank() != 4)
				return;
			if (Group != 1)
				return;
			if (bHasDilation)
				return;

			// Don't transpose if Winograd might be possible
			TArrayView<const uint32> WeightsShape = Weights->GetShape().GetData();
			bool IsKernel3x3 = WeightsShape[2] == 3 && WeightsShape[3] == 3;
			check(Strides.Num() == 2);
			bool AreStrides1 = Strides[0] == 1 && Strides[1] == 1;
			if (IsKernel3x3 && AreStrides1)
				return;

			//Transpose from CwCiHwWw to HwWwCiCw
			if (CPUHelper::Transpose::TransposePreparedData(*Weights, {2,3,1,0} ))
			{
				bAreWeightsTransposed = true;
			}
		};

	private:
		template<int divider>
		static int RoundUp(int value)
		{
			return FMath::DivideAndRoundUp(value, divider) * divider;
		}
	};

	bool ValidateConvOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("auto_pad"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddOptional(TEXT("dilations"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("group"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("kernel_shape"), ENNERuntimeRDGDataAttributeDataType::Int32Array); // idea: cross check input weight shape with this attribute if present
		AttributeValidator.AddOptional(TEXT("pads"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("strides"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("__UE__WinogradPrecision"), ENNERuntimeRDGDataAttributeDataType::Int32);

		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddOptional();
		bIsValid &= InputValidator.Validate(InputTypes);

		for (ENNETensorDataType InputType : InputTypes)
		{
			if (InputType != ENNETensorDataType::None && InputType != InputTypes[0])
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Conv: All input tensor data types have to match each other");
				return false;
			}
		}
		return bIsValid;
	}

	bool RegisterConvOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Conv"), TEXT("Onnx")}, 1}, FConv::Create, ValidateConvOperator);
		Registry.OpAdd({{TEXT("Conv"), TEXT("Onnx")}, 11}, FConv::Create, ValidateConvOperator);
		return true;
	}

} // UE::NNERuntimeRDG::Private::Hlsl
