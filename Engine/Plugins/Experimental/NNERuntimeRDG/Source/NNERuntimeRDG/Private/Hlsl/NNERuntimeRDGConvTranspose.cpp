// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGConvTranspose.h"

#include "NNEHlslShadersConvTransposeCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorConvTranspose, TEXT("NNE.Operator.Hlsl.ConvTranspose"));

	using EConvTransposeAutoPad = UE::NNEHlslShaders::Internal::EConvTransposeAutoPad;

	/**
	 * ConvTranspose operator implementation
	 */
	class FConvTranspose : public FOperatorHlsl
	{
	public:

		static FOperatorHlsl* Create()
		{
			return new FConvTranspose();
		}

		virtual ~FConvTranspose() = default;

	private:

		FConvTranspose() {}

		int32 NumDimensions = 0;

		EConvTransposeAutoPad AutoPad = EConvTransposeAutoPad::NOTSET;
		TArray<int32> Dilations;
		int32 Group = 1;
		TArray<int32> OutputPadding;
		TArray<int32> Pads;
		TArray<int32> Strides;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);
			
			const NNE::FTensorShape& Input = InputTensors[0]->GetShape();
			const NNE::FTensorShape& Weights = InputTensors[1]->GetShape();

			
			const TArray<int32> OutputShapeData = NNEHlslShaders::Internal::FConvTransposeCS::GetOutputShape(Input.GetData(), Weights.GetData(), AutoPad, Dilations, Strides, Pads, OutputPadding, Group);
			const NNE::FSymbolicTensorShape OutputShape = NNE::FSymbolicTensorShape::Make(OutputShapeData);
			if (!OutputShape.IsConcrete())
			{
				return -1;
			}
			OutputTensors[0]->SetShape(NNE::FTensorShape::MakeFromSymbolic(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensorDescs.Num() >= 2 && InputTensorDescs.Num() <= 3);
			check(OutputTensorDescs.Num() == 1);

            const NNE::FTensorDesc& Input = InputTensorDescs[0];
			const NNE::FTensorDesc& Weights = InputTensorDescs[1];
			const NNE::FTensorDesc& Output = OutputTensorDescs[0];

			if (Input.GetShape().Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ConvTranspose: First input should be at least of rank 2");
				return false;
			}
			if (Weights.GetShape().Rank() != Input.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ConvTranspose: First and second inputs should be of same ranks");
				return false;
			}
			if (Output.GetShape().Rank() != Input.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ConvTranspose: First input and output should be of same ranks");
				return false;
			}

			NumDimensions = Input.GetShape().Rank() - 2;

			TArray<int32> DilationsOrStridesDefault;
			DilationsOrStridesDefault.Init(1, NumDimensions);

			FConvTransposeCS::LexFromString(AutoPad, *Attributes.GetValue<FString>(TEXT("auto_pad")));
			Dilations = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("dilations"), DilationsOrStridesDefault);
			Group = Attributes.GetValueOrDefault<int32>(TEXT("group"), 1);
			
			TArray<int32> OutputPaddingDefault;
			OutputPaddingDefault.Init(0, NumDimensions);
			OutputPadding = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("output_padding"), OutputPaddingDefault);
			
			if (AutoPad == EConvTransposeAutoPad::NOTSET)
			{
				TArray<int32> PadsDefault;
				PadsDefault.Init(1, 2 * NumDimensions);

				Pads = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("pads"), PadsDefault);
			}
			Strides = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("strides"), DilationsOrStridesDefault);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			constexpr EConvTransposeAlgorithm Algorithm = EConvTransposeAlgorithm::SharedMemory;
			constexpr EConvTransposeGroupSize GroupSize = EConvTransposeGroupSize::Size256;

			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Weights = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];
			const FTensorRDG* Bias = nullptr;
			bool HasBias = false;

			if (InputTensors.Num() == 3) {
				HasBias = true;
				check(InputTensors[2] != nullptr);
				Bias = InputTensors[2];
			}

			check(Input.GetShape().Rank() > 2);
			check(Weights.GetShape().Rank() == Input.GetShape().Rank());
			check(Output.GetShape().Rank() == Input.GetShape().Rank());
			check(NumDimensions == (Input.GetShape().Rank() - 2));

			TArray<int32> OutputShape = FConvTransposeCS::GetOutputShape(Input.GetShape().GetData(), Weights.GetShape().GetData(), AutoPad, Dilations, Strides, Pads, OutputPadding, Group);

			// Set parameters
			FConvTransposeCS::FParameters* Params = GraphBuilder.AllocParameters<FConvTransposeCS::FParameters>();
			FConvTransposeCS::FillInParameters(GroupSize, Input.GetShape().GetData(), Weights.GetShape().GetData(), HasBias, AutoPad, Group, Dilations,Strides, Pads, OutputPadding, *Params);
			Params->X = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			Params->W = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Weights.GetBuffer(), PF_R32_FLOAT));
			if (HasBias) {
				Params->B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias->GetBuffer(), PF_R32_FLOAT));
			}
			Params->Y = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FConvTransposeCS::FPermutationDomain PermutationVector;

			PermutationVector.Set<FConvTransposeCS::FConvTransposeAlgorithm>(Algorithm);
			PermutationVector.Set<FConvTransposeCS::FConvTransposeGroupSize>(GroupSize);
			PermutationVector.Set<FConvTransposeCS::FConvTransposeNumStackDimensions>(NumDimensions);
			PermutationVector.Set<FConvTransposeCS::FConvTransposeNumReadsPerThread>(FConvTransposeCS::GetNumReadsPerThread(GroupSize, Weights.GetShape().GetData(), Dilations, Strides));
			PermutationVector.Set<FConvTransposeCS::FConvTransposeHasB>(HasBias);
			TShaderMapRef<FConvTransposeCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorConvTranspose, "NNE.Operator.Hlsl.ConvTranspose");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.ConvTranspose.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				FConvTransposeCS::GetGroupCount(OutputShape, FConvTransposeCS::GetGroupShape(GroupSize, NumDimensions)));		
		}
	};

	bool ValidateConvTransposeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("auto_pad"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddOptional(TEXT("dilations"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("group"), ENNERuntimeRDGDataAttributeDataType::Int32);
		//AttributeValidator.AddOptional(TEXT("kernel_shape"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("output_padding"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		//AttributeValidator.AddOptional(TEXT("output_shape"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("pads"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("strides"), ENNERuntimeRDGDataAttributeDataType::Int32Array);

		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddOptional();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	bool RegisterConvTransposeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("ConvTranspose"), TEXT("Onnx")}, 1}, FConvTranspose::Create, ValidateConvTransposeOperator);
		Registry.OpAdd({{TEXT("ConvTranspose"), TEXT("Onnx")}, 11}, FConvTranspose::Create, ValidateConvTransposeOperator);
		return true;
	}

} // UE::NNERuntimeRDG::Private::Hlsl
