// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGCumSum.h"
#include "NNEHlslShadersCumSumCS.h"

#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGCumSum.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorCumSum, TEXT("NNE.Operator.Hlsl.CumSum"));

	/**
	 * CumSum operator implementation
	 */
	template<int Version>
	class FCumSumOperator : public FOperatorHlsl
	{

	public:

		FCumSumOperator() = default;
		virtual ~FCumSumOperator() = default;

	private:

		int32 Axis;
		FIntVector ThreadGroupCount;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);

			const FTensorRef AxisTensor = InputTensors[1];
			if(!AxisTensor->HasPreparedData())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Tensor `axis` (name: %ls) must be CPU constant.", *InputTensors[1]->GetName());
				return -1;
			}
			
			if(AxisTensor->GetVolume() != 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Tensor `axis` (name: %ls) must be 0-D.", *InputTensors[1]->GetName());
				return -1;
			}

			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> TempAxisArray;
			OperatorHelper::GetInt32ArrayFromConstTensor(TempAxisArray, AxisTensor);
			check(TempAxisArray.Num() == 1);
			Axis = TempAxisArray[0];

			const NNE::FTensorShape& InputShape = InputTensors[0]->GetShape();
			
			const int32 InputRank = InputShape.Rank();

			if (Axis > InputRank || Axis  < -InputRank)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Tensor 'axis' should contain a value in the range [-r,r] with r being the rank of the input (name: %ls) however got %d while rank is %d.", *InputTensors[0]->GetName(), Axis, InputRank);
				return -1;
			}

			if (Axis  < 0)
			{
				Axis += InputRank;
			}

			auto ComputeThreadGroupCount = [Axis=Axis](const FTensor& InputTensor) -> FIntVector
			{
				FIntVector OutThreadGroupCount;
				TConstArrayView<uint32> InputShape = InputTensor.GetShape().GetData();

				const uint32 PrefixSumPartitionSize = (uint32) (NNEHlslShaders::Internal::FCumSumConstants::THREADGROUP_SIZE * NNEHlslShaders::Internal::FCumSumConstants::VALUES_PER_THREAD);

				OutThreadGroupCount.X = FMath::DivideAndRoundUp(InputShape[Axis], PrefixSumPartitionSize);

				if(OutThreadGroupCount.X > GRHIMaxDispatchThreadGroupsPerDimension.X)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Input tensor (name: %ls) has axis dimension greater than %d. This is not supported.", 
						*InputTensor.GetName(), GRHIMaxDispatchThreadGroupsPerDimension.X);
					return FIntVector{};
				}
				
				int32 NumElemBeforeAxis = 1;
				int32 NumElemAfterAxis = 1;
				for (int32 DimIdx = 0; DimIdx < Axis; ++DimIdx)
				{
					NumElemBeforeAxis *= InputShape[DimIdx];
				}
				for (int32 DimIdx = Axis + 1; DimIdx < InputShape.Num(); ++DimIdx)
				{
					NumElemAfterAxis *= InputShape[DimIdx];
				}
				
				OutThreadGroupCount.Y = NumElemBeforeAxis;
				if(OutThreadGroupCount.Y > GRHIMaxDispatchThreadGroupsPerDimension.Y)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Input tensor (name: %ls) has number of elements before axis greater than %d. This is not supported.", 
						*InputTensor.GetName(), GRHIMaxDispatchThreadGroupsPerDimension.X);
					return FIntVector{};
				}
				OutThreadGroupCount.Z = NumElemAfterAxis;
				if(OutThreadGroupCount.Z > GRHIMaxDispatchThreadGroupsPerDimension.Z)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Input tensor (name: %ls) has number of elements after axis greater than %d. This is not supported.", 
						*InputTensor.GetName(), GRHIMaxDispatchThreadGroupsPerDimension.Z);
					return FIntVector{};
				}

				return OutThreadGroupCount;
			};

			ThreadGroupCount = ComputeThreadGroupCount(*InputTensors[0]);
			if(ThreadGroupCount == FIntVector{})
			{
				return -1;
			}

			OutputTensors[0]->SetShape(InputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();
			
			int32 Exclusive = Attributes.GetValueOrDefault<int32>(TEXT("exclusive"), 0);
			if(Exclusive == 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Attribute `exclusive` not yet supported.");
				return false;
			}

			int32 Reverse = Attributes.GetValueOrDefault<int32>(TEXT("reverse"), 0);
			if(Reverse == 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "CumSum: Attribute `reverse` not yet supported.");
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != FTensorRDGRef{});
			check(OutputTensors[0] != FTensorRDGRef{});
			check(ThreadGroupCount != FIntVector{});

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorCumSum, "NNE.Operator.Hlsl.CumSum");

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			const uint32 ThreadGroupCountTotal = ThreadGroupCount.X * ThreadGroupCount.Y * ThreadGroupCount.Z;
			const uint32 NumParallelScans = ThreadGroupCount.Y * ThreadGroupCount.Z;

			const FRDGBufferDesc GPIBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumParallelScans);
			FRDGBufferRef GPIBuffer = GraphBuilder.CreateBuffer(GPIBufferDesc, TEXT("NNE.Operator.Hlsl.CumSum.GPIBuffer"));

			const FRDGBufferDesc PartitionDescriptorBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FPartitionDescriptor), ThreadGroupCountTotal);
			FRDGBufferRef PartitionDescriptorBuffer = GraphBuilder.CreateBuffer(PartitionDescriptorBufferDesc, TEXT("NNE.Operator.Hlsl.CumSum.PartitionDescriptorBuffer"));

			{
				TInitCumSumCS::FParameters* Params = GraphBuilder.AllocParameters<TInitCumSumCS::FParameters>();
				Params->GlobalPartitionIndex = GraphBuilder.CreateUAV(GPIBuffer);
				Params->PartitionDescriptor = GraphBuilder.CreateUAV(PartitionDescriptorBuffer);
				Params->NumThreadGroupsPerScan = ThreadGroupCount.X;
				Params->NumThreadGroupsY = ThreadGroupCount.Y;
				Params->NumThreadGroupsZ = ThreadGroupCount.Z;

				const FIntVector InitThreadGroupCount = ComputeElementWiseThreadGroups(ThreadGroupCountTotal, FCumSumConstants::INIT_THREADGROUP_SIZE);
				Params->NumInitThreadGroups = InitThreadGroupCount.X;

				TInitCumSumCS::FPermutationDomain PermutationVector;

				TShaderMapRef<TInitCumSumCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.InitCumSum.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					InitThreadGroupCount);
				
			}
			
			{
				const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
				const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
				
				// Set parameters
				TCumSumCS::FParameters* Params = GraphBuilder.AllocParameters<TCumSumCS::FParameters>();
				Params->Input = InputSRV;
				Params->Output = OutputUAV;
				Params->GlobalPartitionIndex = GraphBuilder.CreateUAV(GPIBuffer);
				Params->PartitionDescriptor = GraphBuilder.CreateUAV(PartitionDescriptorBuffer);
				
				Params->NumThreadGroupsPerScan = ThreadGroupCount.X;
				Params->NumThreadGroupsY = ThreadGroupCount.Y;
				Params->NumThreadGroupsZ = ThreadGroupCount.Z;
				Params->NumScanValues = Input.GetShape().GetData()[Axis];
				Params->Axis = (uint32) Axis;
				// Compute Axis' stride
				{
					uint32 Stride = 1;
					for (int32 Idx = Input.GetShape().Rank() - 1; Idx > Axis; --Idx)
					{
						Stride *= Input.GetShape().GetData()[Idx];
					}
					Params->AxisStride = Stride;
				}

				TCumSumCS::FPermutationDomain PermutationVector;

				TShaderMapRef<TCumSumCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.CumSum.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					ThreadGroupCount);

			}

		}
	};

	template<int Version>
	bool ValidateCumSumOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		// This matches versions 11 and 14 of the CumSum operator
		//
		// https://github.com/onnx/onnx/blob/main/docs/Changelog.md#CumSum-11
		// https://github.com/onnx/onnx/blob/main/docs/Changelog.md#CumSum-14

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("exclusive"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("reverse"), ENNERuntimeRDGDataAttributeDataType::Int32);
		
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 1);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32, /*TemplateIdx*/ 1);
		InputValidator.AddRequired();
		InputValidator.AddRequired(/*TemplateIdx*/ 1);
		
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<int Version>
	FOperatorHlsl* CreateCumSumOperator()
	{
		return new FCumSumOperator<Version>();
	}

	bool RegisterCumSumOperator(FOperatorRegistryHlsl& Registry)
	{
		using namespace UE::NNEHlslShaders::Internal;

		// Note: CumSum is currently not working on Mac.
		#if !PLATFORM_MAC
			Registry.OpAdd({ {TEXT("CumSum"), TEXT("Onnx")}, 11}, CreateCumSumOperator<11>, ValidateCumSumOperator<11>);
			Registry.OpAdd({ {TEXT("CumSum"), TEXT("Onnx")}, 14}, CreateCumSumOperator<14>, ValidateCumSumOperator<14>);
		#endif

		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
