// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGDepthToSpace.h"

#include "NNERuntimeRDGHlslHelper.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersTransposeCS.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorDepthToSpace, TEXT("NNE.Operator.Hlsl.DepthToSpace"));

	/**
	 * DepthToSpace operator implementation
	 */
	class FDepthToSpace : public FOperatorHlsl
	{
	public:

		FDepthToSpace() {}
		virtual ~FDepthToSpace() = default;

	private:

		enum class EMode : uint8
		{
			DCR = 0,
			CRD,
			MAX
		};

		EMode ModeFromString(const TCHAR* StringVal)
		{
			EMode OutValue = EMode::MAX;
			if (FCString::Stricmp(StringVal, TEXT("DCR")) == 0) OutValue = EMode::DCR;
			else if (FCString::Stricmp(StringVal, TEXT("CRD")) == 0) OutValue = EMode::CRD;

			return OutValue;
		}

		using TShapeArray = TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>>;

		TShapeArray PreTransposeShape;
		TShapeArray TransposePerm;
		TShapeArray PostTransposeShape;
		uint32 BlockSize;
		EMode Mode;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			TConstArrayView<uint32> InputShape = X.GetShape().GetData();

			if(InputShape.Num() != 4)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "DepthToSpace: input tensor (name: %ls) doesn't have [N,C,H,W] format.", *X.GetName());
				return false;
			}

			const uint32 NumBatches = InputShape[0];
			const uint32 NumDepths = InputShape[1];
			const uint32 Height = InputShape[2];
			const uint32 Width = InputShape[3];
			const uint32 NewNumDepths = NumDepths / (BlockSize*BlockSize);

			PreTransposeShape = Mode == EMode::DCR ?
					TShapeArray{NumBatches, BlockSize, BlockSize, NewNumDepths, Height, Width} 
				:	
					TShapeArray{NumBatches, NewNumDepths, BlockSize, BlockSize, Height, Width} 
				;

			PostTransposeShape.SetNumUninitialized(6);
			for (int32 i = 0; i < 6; ++i)
			{
				PostTransposeShape[i] = PreTransposeShape[TransposePerm[i]];
			}

			TShapeArray OutputShape = {NumBatches, NewNumDepths, Height * BlockSize, Width * BlockSize};
			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			const NNE::FTensorDesc& InputDesc = InputTensorDescs[0];
			const int32 InputRank = InputDesc.GetShape().Rank();

			const FNNERuntimeRDGDataAttributeValue* BlockSizeAttr = Attributes.GetAttributeValue(TEXT("blocksize"));
			if(BlockSizeAttr == nullptr)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "DepthToSpace: blocksize attribute is required.");
				return false;
			}

			BlockSize = BlockSizeAttr->GetValue<int32>();	
			Mode = ModeFromString(*Attributes.GetValueOrDefault<FString>(TEXT("mode"), TEXT("DCR")));
			if(Mode == EMode::MAX)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "DepthToSpace: invalid mode.");
				return false;
			}

			TransposePerm = Mode == EMode::DCR ? 
					TShapeArray{0, 3, 4, 1, 5, 2}
				:
					TShapeArray{0, 1, 4, 2, 5, 3}
				;
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];
			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FTransposeConstants::NUM_GROUP_THREADS);

			// Set parameters
			FTransposeCS::FParameters* Params = GraphBuilder.AllocParameters<FTransposeCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FTransposeConstants::NUM_GROUP_THREADS;

			auto FillTensorInfoStrideFromShape = [](const TShapeArray& Shape, FTensorInfoParam& OutTensorInfo, const int32 Idx)
			{
				OutTensorInfo[Shape.Num() - 1][Idx] = 1;
				for (int32 i = Shape.Num() - 2; i >= 0; --i)
				{
					OutTensorInfo[i][Idx] = OutTensorInfo[i + 1][Idx] * Shape[i + 1];
				}
			};

			FillTensorInfoStrideFromShape(PostTransposeShape, Params->TensorInfo, 0);
			FillTensorInfoStrideFromShape(PreTransposeShape, Params->TensorInfo, 1);
			for (int32 i = 0; i < TransposePerm.Num(); ++i)
			{
				Params->TensorInfo[i][2] = Params->TensorInfo[TransposePerm[i]][1];
			}

			FTransposeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FTransposeCS::FTransposeNumDimensions>(PostTransposeShape.Num());

			TShaderMapRef<FTransposeCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorDepthToSpace, "NNE.Operator.Hlsl.DepthToSpace");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Transpose.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateDepthToSpaceOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddRequired(TEXT("blocksize"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("mode"), ENNERuntimeRDGDataAttributeDataType::String);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateDepthToSpaceOperator()
	{
		return new FDepthToSpace();
	}

	bool RegisterDepthToSpaceOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("DepthToSpace"), TEXT("Onnx")}, 1}, CreateDepthToSpaceOperator, ValidateDepthToSpaceOperator);
		Registry.OpAdd({{TEXT("DepthToSpace"), TEXT("Onnx")}, 11}, CreateDepthToSpaceOperator, ValidateDepthToSpaceOperator);
		Registry.OpAdd({{TEXT("DepthToSpace"), TEXT("Onnx")}, 13}, CreateDepthToSpaceOperator, ValidateDepthToSpaceOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
