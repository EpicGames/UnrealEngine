// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGDropout.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorDropout, TEXT("NNE.Operator.Hlsl.Dropout"));

	/**
	 * Dropout operator implementation
	 */
	class FDropout : public FOperatorHlsl
	{
	public:

		FDropout() {}
		virtual ~FDropout() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];

			OutputTensors[0]->SetShape(X.GetShape());
			if (X.HasPreparedData())
			{
				OutputTensors[0]->SetPreparedData<uint8>(X.GetPreparedData<uint8>());
			}

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1 || OutputTensorDescs.Num() == 2);

			if (OutputTensorDescs.Num() == 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Dropout: Is only supported in inference mode at the moment, without the ability to output a 'mask'.");
				return false;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1 || OutputTensors.Num() == 2);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorDropout, "NNE.Operator.Hlsl.Dropout");

			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Data.GetBuffer());
		}
	};

	template<int Version>
	bool ValidateDropoutOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputDropouts)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		if constexpr (Version >= 12)
		{
			AttributeValidator.AddOptional(TEXT("seed"), ENNERuntimeRDGDataAttributeDataType::Int32);
		}
		else
		{
			static_assert(Version >= 7, "Minimum supported version for operator Dropout is 7!");
			AttributeValidator.AddOptional(TEXT("ratio"), ENNERuntimeRDGDataAttributeDataType::Float);
		}
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddRequired();

		if (InputTypes.Num() > 1)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Dropout: Is only supported in inference mode at the moment, 'ratio' and 'training_mode' will be ignored.");
		}

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateDropoutOperator()
	{
		return new FDropout();
	}

	bool RegisterDropoutOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Dropout"), TEXT("Onnx")}, 7}, CreateDropoutOperator, ValidateDropoutOperator<7>);
		Registry.OpAdd({{TEXT("Dropout"), TEXT("Onnx")}, 10}, CreateDropoutOperator, ValidateDropoutOperator<10>);
		Registry.OpAdd({{TEXT("Dropout"), TEXT("Onnx")}, 12}, CreateDropoutOperator, ValidateDropoutOperator<12>);
		Registry.OpAdd({{TEXT("Dropout"), TEXT("Onnx")}, 13}, CreateDropoutOperator, ValidateDropoutOperator<13>);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
