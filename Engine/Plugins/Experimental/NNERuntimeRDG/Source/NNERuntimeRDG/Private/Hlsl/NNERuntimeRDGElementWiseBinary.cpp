// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGElementWiseBinary.h"

#include "Helper/NNERuntimeRDGHelperElementWiseBinary.h"
#include "NNEHlslShadersElementWiseBinaryCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorElementWiseBinary, TEXT("NNE.Operator.Hlsl.ElementWise.Binary"));

	using TElementWiseBinaryCS = typename UE::NNEHlslShaders::Internal::TElementWiseBinaryCS;
	using FElementWiseBinaryConstants = UE::NNEHlslShaders::Internal::FElementWiseBinaryConstants;
	using EElementWiseBinaryOperatorType = UE::NNEHlslShaders::Internal::EElementWiseBinaryOperatorType;

	/**
	 * Binary element-wise operator implementation
	 */
	template<EElementWiseBinaryOperatorType OpType>
	class TElementWiseBinary : public FOperatorHlsl
	{
	public:

		TElementWiseBinary() {}
		virtual ~TElementWiseBinary() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			
			const NNE::FTensorShape& LHSInput = InputTensors[0]->GetShape();
			const NNE::FTensorShape& RHSInput = InputTensors[1]->GetShape();
			const int32 OutputRank = FMath::Max(LHSInput.Rank(), RHSInput.Rank());
			TArray<uint32> OutputShapeData;
			
			OutputShapeData.SetNumUninitialized(OutputRank);
			
			for (int32 i = 0; i < OutputRank; ++i)
			{
				int32 LHSIndex = LHSInput.Rank() - 1 - i;
				int32 RHSIndex = RHSInput.Rank() - 1 - i;
				int32 LHSValue = LHSIndex >= 0 ? LHSInput.GetData()[LHSIndex] : 1;
				int32 RHSValue = RHSIndex >= 0 ? RHSInput.GetData()[RHSIndex] : 1;
				if (LHSValue != RHSValue && LHSValue != 1 && RHSValue != 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ElementWiseBinary: Could not compute the shape, input shapes are not compatible");
					return -1;
				}
				int32 OutputValue = FMath::Max(LHSValue, RHSValue);
				OutputShapeData[OutputRank - 1 - i] = OutputValue;
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			
			OutputTensors[0]->SetShape(OutputShape);

			CPUHelper::ElementWiseBinary::Apply(OpType, *InputTensors[0], *InputTensors[1], *OutputTensors[0]);

			if (OutputTensors[0]->GetDataType() != ENNETensorDataType::Float && !OutputTensors[0]->HasPreparedData())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ElementWiseBinary: Output tensor could not be made constant nor it was of float type. Only floats are supported at the moment.");
				return -1;
			}
			
			return 0;
		}
		
		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);
			check(InputTensorDescs[0].GetDataType() == InputTensorDescs[1].GetDataType());
			check(InputTensorDescs[0].GetDataType() == OutputTensorDescs[0].GetDataType());
		
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& LHSInput = *InputTensors[0];
			const FTensorRDG& RHSInput = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];

			FRDGBufferSRVRef LHSInputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LHSInput.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferSRVRef RHSInputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RHSInput.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FElementWiseBinaryConstants::NUM_GROUP_THREADS);

			// Set parameters
			TElementWiseBinaryCS::FParameters* Params = GraphBuilder.AllocParameters<TElementWiseBinaryCS::FParameters>();
			Params->LHSInput = LHSInputSRV;
			Params->RHSInput = RHSInputSRV;
			Params->Output = OutputUAV;
			FillTensorStrideForBroadcastShaderParameters(LHSInput, Output.GetShape().Rank(), Params->TensorInfo, 0);
			FillTensorStrideForBroadcastShaderParameters(RHSInput, Output.GetShape().Rank(), Params->TensorInfo, 1);
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 2);
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FElementWiseBinaryConstants::NUM_GROUP_THREADS;

			TElementWiseBinaryCS::FPermutationDomain PermutationVector;

			PermutationVector.Set<TElementWiseBinaryCS::FOperatorType>(OpType);
			PermutationVector.Set<TElementWiseBinaryCS::FBinaryNumDimensions>(Output.GetShape().Rank());

			TShaderMapRef<TElementWiseBinaryCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
		
			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorElementWiseBinary, "NNE.Operator.Hlsl.ElementWise.Binary");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.ElementWise.Binary.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateElementWiseBinaryOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<EElementWiseBinaryOperatorType OpType>
	FOperatorHlsl* CreateElementWiseBinaryOperator()
	{
		return new TElementWiseBinary<OpType>();
	}

	bool RegisterElementWiseBinaryOperators(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
#define OP(Name, Version) Registry.OpAdd({{TEXT(#Name), TEXT("Onnx")}, Version}, CreateElementWiseBinaryOperator<EElementWiseBinaryOperatorType::Name>, ValidateElementWiseBinaryOperator);
#define OP_ALL_VERSIONS(Name) \
OP(Name, 6) \
OP(Name, 7) \
OP(Name, 13) \
OP(Name, 14)
		
		OP_ALL_VERSIONS(Add)
		//OP(And, 1)
		//OP(And, 7)
		OP_ALL_VERSIONS(Div)
		//OP(Equal, 1)
		//OP(Equal, 7)
		//OP(Equal, 11)
		//OP(Equal, 13)
		//OP(Equal, 19)
		//OP(Greater, 1)
		//OP(Greater, 7)
		//OP(Greater, 9)
		//OP(Greater, 13)
		//OP(GreaterOrEqual, 12)
		//OP(GreaterOrEqual, 16)
		//OP(Less, 1)
		//OP(Less, 7)
		//OP(Less, 9)
		//OP(Less, 13)
		//OP(LessOrEqual, 12)
		//OP(LessOrEqual, 16)
		OP(Mod, 10)
		OP(Mod, 13)
		OP_ALL_VERSIONS(Mul)
		//OP(Or, 1)
		//OP(Or, 7)
		OP(Prelu, 6)
		OP(Prelu, 7)
		OP(Prelu, 9)
		OP(Prelu, 16)
		OP(Pow, 7)
		OP(Pow, 12)
		OP(Pow, 13)
		OP(Pow, 15)
		OP_ALL_VERSIONS(Sub)
		//OP(Xor, 1)
		//OP(Xor, 7)
#undef OP_ALL_VERSIONS
#undef OP

		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
