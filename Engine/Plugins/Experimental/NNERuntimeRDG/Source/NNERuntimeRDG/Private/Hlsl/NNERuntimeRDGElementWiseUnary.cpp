// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGElementWiseUnary.h"

#include "Helper/NNERuntimeRDGHelperElementWiseUnary.h"
#include "NNEHlslShadersElementWiseUnaryCS.h"
#include "NNEHlslShadersTypeHelper.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorElementWiseUnary, TEXT("NNE.Operator.Hlsl.ElementWise.Unary"));

	using TElementWiseUnaryCS = typename UE::NNEHlslShaders::Internal::TElementWiseUnaryCS;
	using FElementWiseUnaryConstants = UE::NNEHlslShaders::Internal::FElementWiseUnaryConstants;
	using EElementWiseUnaryOperatorType = UE::NNEHlslShaders::Internal::EElementWiseUnaryOperatorType;

	/**
	 * Unary element-wise operator implementation
	 */
	template<EElementWiseUnaryOperatorType OpType>
	class TElementWiseUnary : public FOperatorHlsl
	{
	public:

		TElementWiseUnary(int InVersion) : Version(InVersion) {};
		virtual ~TElementWiseUnary() = default;

	private:

		float Alpha = 0.0f;
		float Beta = 0.0f;
		float Gamma = 0.0f;

		int Version = 0;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check((OpType == EElementWiseUnaryOperatorType::Clip && Version >= 11) ? (InputTensors.Num() >= 1 && InputTensors.Num() <= 3) : InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			
			OutputTensors[0]->SetShape(InputTensors[0]->GetShape());

			const FTensor& X = *InputTensors[0];

			if(OpType == EElementWiseUnaryOperatorType::Clip && Version >= 11 && InputTensors.Num() >= 2)
			{
				{
					const FTensor& MinTensor = *InputTensors[1];
					if(MinTensor.HasPreparedData())
					{
						Alpha = MinTensor.GetPreparedData<float>()[0];
					}
				}

				if(InputTensors.Num() == 3)
				{
					const FTensor& MaxTensor = *InputTensors[2];
					if(MaxTensor.HasPreparedData())
					{
						Beta = MaxTensor.GetPreparedData<float>()[0];
					}
				}
			}

			CPUHelper::ElementWiseUnary::Apply(OpType, X, Alpha, Beta, Gamma, *OutputTensors[0]);
			
			return 0;
		}

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			Alpha = Attributes.GetValueOrDefault(TEXT("alpha"), Alpha);
			Beta = Attributes.GetValueOrDefault(TEXT("beta"), Beta);
			Gamma = Attributes.GetValueOrDefault(TEXT("gamma"), Gamma);
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);
			FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputTensors[0]->GetBuffer(), TensorDataTypeToPixelFormat(InputTensors[0]->GetDataType())));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputTensors[0]->GetBuffer(), TensorDataTypeToPixelFormat(OutputTensors[0]->GetDataType())));
		
			int32 NumElements = OutputTensors[0]->GetVolume();
			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(NumElements, FElementWiseUnaryConstants::NUM_GROUP_THREADS);

			// Set parameters
			TElementWiseUnaryCS::FParameters* Params = GraphBuilder.AllocParameters<TElementWiseUnaryCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Alpha = Alpha;
			Params->Beta = Beta;
			Params->Gamma = Gamma;
			Params->Num = NumElements;
			Params->ThreadCountX = ThreadGroupCount.X * FElementWiseUnaryConstants::NUM_GROUP_THREADS;

			TElementWiseUnaryCS::FPermutationDomain PermutationVector;

			PermutationVector.Set<TElementWiseUnaryCS::FOperatorType>(OpType);
			PermutationVector.Set<TElementWiseUnaryCS::FAlphaOnGPU>(false);
			PermutationVector.Set<TElementWiseUnaryCS::FBetaOnGPU>(false);

			if(OpType == EElementWiseUnaryOperatorType::Clip && Version >= 11 && InputTensors.Num() >= 2)
			{
				{
					const FTensor& MinTensor = *InputTensors[1];
					if(!MinTensor.HasPreparedData())
					{
						PermutationVector.Set<TElementWiseUnaryCS::FAlphaOnGPU>(true);
						FRDGBufferSRVRef AlphaSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputTensors[1]->GetBuffer(), PF_R32_FLOAT));
						Params->AlphaTensor = AlphaSRV;
					}
				}

				if(InputTensors.Num() == 3)
				{
					const FTensor& MaxTensor = *InputTensors[2];
					if(!MaxTensor.HasPreparedData())
					{
						PermutationVector.Set<TElementWiseUnaryCS::FBetaOnGPU>(true);
						FRDGBufferSRVRef BetaSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputTensors[2]->GetBuffer(), PF_R32_FLOAT));
						Params->BetaTensor = BetaSRV;
					}
				}
			}

			TShaderMapRef<TElementWiseUnaryCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorElementWiseUnary, "NNE.Operator.Hlsl.ElementWise.Unary");
		
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.ElementWise.Unary.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	template<> TElementWiseUnary<EElementWiseUnaryOperatorType::Selu>::TElementWiseUnary(int InVersion)
		: Alpha(1.67326319217681884765625f), Beta(0.0f), Gamma(1.05070102214813232421875f), Version(InVersion)
	{
	}

	template<> TElementWiseUnary<EElementWiseUnaryOperatorType::Elu>::TElementWiseUnary(int InVersion)
		: Alpha(1.0f), Beta(0.0f), Gamma(0.0f), Version(InVersion)
	{
	}

	template<> TElementWiseUnary<EElementWiseUnaryOperatorType::HardSigmoid>::TElementWiseUnary(int InVersion)
		: Alpha(0.2f), Beta(0.5f), Gamma(0.0f), Version(InVersion)
	{
	}

	template<> TElementWiseUnary<EElementWiseUnaryOperatorType::LeakyRelu>::TElementWiseUnary(int InVersion)
		: Alpha(0.01f), Beta(0.0f), Gamma(0.0f), Version(InVersion)
	{
	}

	template<> bool TElementWiseUnary<EElementWiseUnaryOperatorType::Clip>::Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes)
	{
		check(Version >= 11 ? (InputTensorDescs.Num() >= 1 && InputTensorDescs.Num() <= 3) : InputTensorDescs.Num() == 1);
		check(OutputTensorDescs.Num() == 1);

		Alpha = Attributes.GetValueOrDefault(TEXT("min"), -3.402823e+38f);
		Beta = Attributes.GetValueOrDefault(TEXT("max"), 3.402823e+38f);
		return true;
	}

	template<EElementWiseUnaryOperatorType OpType, int Version>
	FOperatorHlsl* CreateElementWiseUnaryOperator()
	{
		return new TElementWiseUnary<OpType>(Version);
	}
	
	template<EElementWiseUnaryOperatorType OpType>
	bool ValidateElementWiseUnaryOperatorImpl(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<>
	bool ValidateElementWiseUnaryOperatorImpl<EElementWiseUnaryOperatorType::Selu>(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("alpha"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("gamma"), ENNERuntimeRDGDataAttributeDataType::Float);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<>
	bool ValidateElementWiseUnaryOperatorImpl<EElementWiseUnaryOperatorType::Elu>(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("alpha"), ENNERuntimeRDGDataAttributeDataType::Float);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<>
	bool ValidateElementWiseUnaryOperatorImpl<EElementWiseUnaryOperatorType::HardSigmoid>(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("alpha"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("beta"), ENNERuntimeRDGDataAttributeDataType::Float);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<>
	bool ValidateElementWiseUnaryOperatorImpl<EElementWiseUnaryOperatorType::LeakyRelu>(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("alpha"), ENNERuntimeRDGDataAttributeDataType::Float);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<>
	bool ValidateElementWiseUnaryOperatorImpl<EElementWiseUnaryOperatorType::Clip>(int Version, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		if(Version < 11)
		{
			AttributeValidator.AddOptional(TEXT("min"), ENNERuntimeRDGDataAttributeDataType::Float);
			AttributeValidator.AddOptional(TEXT("max"), ENNERuntimeRDGDataAttributeDataType::Float);
		}
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		if(Version >= 11)
		{
			InputValidator.AddOptional();
			InputValidator.AddOptional();
		}
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<EElementWiseUnaryOperatorType OpType, int Version>
	bool ValidateElementWiseUnaryOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		return ValidateElementWiseUnaryOperatorImpl<OpType>(Version, AttributeMap, InputTypes, InputShapes);
	}
	
	bool RegisterElementWiseUnaryOperators(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
#define OP(Name, Version) Registry.OpAdd({{TEXT(#Name), TEXT("Onnx")}, Version}, CreateElementWiseUnaryOperator<EElementWiseUnaryOperatorType::Name, Version>, ValidateElementWiseUnaryOperator<EElementWiseUnaryOperatorType::Name, Version>);
		OP(Abs, 6)
		OP(Abs, 13)
		OP(Acos, 7)
		OP(Acosh, 9)
		OP(Asin, 7)
		OP(Asinh, 9)
		OP(Atan, 7)
		OP(Atanh, 9)
		//OP(BitShift, 11)
		OP(Ceil, 6)
		OP(Ceil, 13)
		OP(Clip, 6)
		OP(Clip, 11)
		OP(Clip, 12)
		OP(Clip, 13)
		OP(Cos, 7)
		OP(Cosh, 9)
		OP(Elu, 6)
		OP(Erf, 9)
		OP(Erf, 13)
		OP(Exp, 6)
		OP(Exp, 13)
		OP(Floor, 6)
		OP(Floor, 13)
		OP(IsInf, 10)
		OP(IsInf, 20)
		OP(IsNan, 9)
		OP(IsNan, 13)
		OP(IsNan, 20)
		OP(HardSigmoid, 6)
		OP(HardSwish, 14)
		OP(LeakyRelu, 6)
		OP(LeakyRelu, 16)
		OP(Log, 6)
		OP(Log, 13)
		OP(Neg, 6)
		OP(Neg, 13)
		//OP(Not, 1)
		OP(Reciprocal, 6)
		OP(Reciprocal, 13)
		OP(Relu, 6)
		OP(Relu, 13)
		OP(Relu, 14)
		OP(Round, 11)
		OP(Selu, 6)
		OP(Sigmoid, 6)
		OP(Sigmoid, 13)
		OP(Sign, 9)
		OP(Sign, 13)
		OP(Sin, 7)
		OP(Sinh, 9)
		OP(Softplus, 1)
		OP(Softsign, 1)
		OP(Sqrt, 6)
		OP(Sqrt, 13)
		OP(Tan, 7)
		OP(Tanh, 6)
		OP(Tanh, 13)
#undef OP

		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
