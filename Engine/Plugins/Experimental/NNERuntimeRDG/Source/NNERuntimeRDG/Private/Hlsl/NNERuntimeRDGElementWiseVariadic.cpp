// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGElementWiseVariadic.h"

#include "NNEHlslShadersElementWiseVariadicCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorElementWiseVariadic, TEXT("NNE.Operator.Hlsl.ElementWise.Variadic"));

	using TElementWiseVariadicCS = typename UE::NNEHlslShaders::Internal::TElementWiseVariadicCS;
	using FElementWiseVariadicConstants = UE::NNEHlslShaders::Internal::FElementWiseVariadicConstants;
	using EElementWiseVariadicOperatorType = UE::NNEHlslShaders::Internal::EElementWiseVariadicOperatorType;

	void AddOneVariadicOpPass(FRDGBuilder& GraphBuilder, 
		TConstArrayView<FTensorRDGRef> InputTensors,
		const FTensorRDG& OutputTensor,
		bool OutputAsInput,
		EElementWiseVariadicOperatorType OpType,
		float Scale)
	{
		static_assert(FElementWiseVariadicConstants::MAX_NUM_INPUT == 4, "This algorithm need to be adapted to math the shader.");
		check(InputTensors.Num() <= FElementWiseVariadicConstants::MAX_NUM_INPUT);
		check(InputTensors.Num() > 0);

		// SRVs & UAV creations
		FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputTensor.GetBuffer(), PF_R32_FLOAT));
		FRDGBufferSRVRef InputsSRV[FElementWiseVariadicConstants::MAX_NUM_INPUT] = { nullptr, nullptr, nullptr, nullptr };

		for (int32 i = 0; i < InputTensors.Num(); ++i)
		{
			check(InputTensors[i] != nullptr);
			InputsSRV[i] = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputTensors[i]->GetBuffer(), PF_R32_FLOAT));
		}

		// Set parameters
		FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(OutputTensor.GetVolume(), FElementWiseVariadicConstants::NUM_GROUP_THREADS);
		TElementWiseVariadicCS::FParameters* Params = GraphBuilder.AllocParameters<TElementWiseVariadicCS::FParameters>();

		Params->Input0 = InputsSRV[0];
		Params->Input1 = InputsSRV[1];
		Params->Input2 = InputsSRV[2];
		Params->Input3 = InputsSRV[3];
		Params->Output = OutputUAV;
		FillTensorStrideForBroadcastShaderParameters(*InputTensors[0], OutputTensor.GetShape().Rank(), Params->InputTensorInfo, 0);
		if (InputTensors.Num() >= 2)
		{
			FillTensorStrideForBroadcastShaderParameters(*InputTensors[1], OutputTensor.GetShape().Rank(), Params->InputTensorInfo, 1);
		}
		if (InputTensors.Num() >= 3)
		{
			FillTensorStrideForBroadcastShaderParameters(*InputTensors[2], OutputTensor.GetShape().Rank(), Params->InputTensorInfo, 2);
		}
		if (InputTensors.Num() >= 4)
		{
			FillTensorStrideForBroadcastShaderParameters(*InputTensors[3], OutputTensor.GetShape().Rank(), Params->InputTensorInfo, 3);
		}
		FillTensorStrideShaderParameters(OutputTensor, Params->OutputTensorInfo, 0);
		Params->Num = OutputTensor.GetVolume();
		Params->ThreadCountX = ThreadGroupCount.X * FElementWiseVariadicConstants::NUM_GROUP_THREADS;
		Params->Scale = Scale;

		// Shader variation
		TElementWiseVariadicCS::FPermutationDomain PermutationVector;

		PermutationVector.Set<TElementWiseVariadicCS::FOperatorType>(OpType);
		PermutationVector.Set<TElementWiseVariadicCS::FApplyScale>(Scale != 1.0f);
		PermutationVector.Set<TElementWiseVariadicCS::FOutputAsInput>(OutputAsInput);
		PermutationVector.Set<TElementWiseVariadicCS::FNumInput>(InputTensors.Num());
		PermutationVector.Set<TElementWiseVariadicCS::FVariadicNumDimensions>(OutputTensor.GetShape().Rank());

		// Add the pass to RDG
		TShaderMapRef<TElementWiseVariadicCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("NNE.Operator.Hlsl.ElementWise.Variadic.Dispatch"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			Params,
			ThreadGroupCount);
	}

	/**
	 * Variadic Element-wise ML operator implementation
	 */
	template<EElementWiseVariadicOperatorType OpType>
	class TElementWiseVariadic : public FOperatorHlsl
	{
	public:

		TElementWiseVariadic() {}
		virtual ~TElementWiseVariadic() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() > 0);
			check(OutputTensors.Num() == 1);

			const int32 NumInput = InputTensors.Num();
			int32 OutputRank = 0;
			for (int32 i = 0; i < NumInput; ++i)
			{
				OutputRank = FMath::Max(OutputRank, InputTensors[i]->GetShape().Rank());
			}

			TArray<uint32> OutputShapeData;

			OutputShapeData.SetNumUninitialized(OutputRank);

			for (int32 i = 0; i < OutputRank; ++i)
			{
				int32 OutputValue = 1;
				for (int32 InputIdx = 0; InputIdx < NumInput; ++InputIdx)
				{
					int32 InputIndex = InputTensors[InputIdx]->GetShape().Rank() - 1 - i;
					int32 InputValue = InputIndex >= 0 ? InputTensors[InputIdx]->GetShape().GetData()[InputIndex] : 1;
					if (InputValue != OutputValue && InputValue != 1 && OutputValue != 1)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ElementWiseVariadic: Could not compute shape for element wise variadic op, input shapes are not compatible");
						return -1;
					}
					OutputValue = FMath::Max(InputValue, OutputValue);
				}
				OutputShapeData[OutputRank - 1 - i] = OutputValue;
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);

			OutputTensors[0]->SetShape(OutputShape);

			return 0;
		}
		
		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() > 0);
			check(OutputTensorDescs.Num() == 1);
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InInputTensors, TConstArrayView<FTensorRDGRef> InOutputTensors) override
		{
			check(InInputTensors.Num() > 0);
			check(InOutputTensors.Num() == 1);
			check(InOutputTensors[0] != nullptr);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorElementWiseVariadic, "NNE.Operator.Hlsl.ElementWise.Variadic");

			FTensorRDGRef PassInputTensors[FElementWiseVariadicConstants::MAX_NUM_INPUT];
			for (int32 InputOffset = 0; InputOffset < InInputTensors.Num(); InputOffset += FElementWiseVariadicConstants::MAX_NUM_INPUT)
			{
				uint32 NumInputLeftToHandle = InInputTensors.Num() - InputOffset;
				uint32 NumInputForPass = FMath::Min(FElementWiseVariadicConstants::MAX_NUM_INPUT, NumInputLeftToHandle);
				bool bIsFirstPass = (InputOffset == 0);
				bool bIsLastPass = (NumInputLeftToHandle <= FElementWiseVariadicConstants::MAX_NUM_INPUT);
				float Scale = 1.0f;

				for (uint32 i = 0; i < NumInputForPass; ++i)
				{
					check(InInputTensors[InputOffset + i] != nullptr);
					PassInputTensors[i] = InInputTensors[InputOffset + i];
				}

				if (OpType == EElementWiseVariadicOperatorType::Mean && bIsLastPass)
				{
					Scale = 1.0f / InInputTensors.Num();
				}
			
				AddOneVariadicOpPass(GraphBuilder,
					MakeArrayView(PassInputTensors, NumInputForPass),
					*InOutputTensors[0],
					!bIsFirstPass,
					OpType, Scale);
			}
		}
	};

	bool ValidateElementWiseVariadicOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		if (InputTypes.Num() == 0)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "ElementWiseVariadic: Operator requires at least 1 input");
			bIsValid = false;
		}
		for (int32 i = 0; i < InputTypes.Num(); ++i)
		{
			if (InputTypes[i] != ENNETensorDataType::Float)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ElementWiseVariadic: Operator input '%d' of type '%d' is not supported, should be float at the moment.", i, int(InputTypes[i]));
				bIsValid = false;
			}
		}
		
		return bIsValid;
	}

	template<EElementWiseVariadicOperatorType OpType>
	FOperatorHlsl* CreateElementWiseVariadicOperator()
	{
		return new TElementWiseVariadic<OpType>();
	}

	bool RegisterElementWiseVariadicOperators(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
#define OP(Name, Version) Registry.OpAdd({{TEXT(#Name), TEXT("Onnx")}, Version}, CreateElementWiseVariadicOperator<EElementWiseVariadicOperatorType::Name>);
#define OP_ALL_VERSIONS(Name) \
OP(Name, 6) \
OP(Name, 8) \
OP(Name, 12) \
OP(Name, 13)
		OP_ALL_VERSIONS(Max)
		OP_ALL_VERSIONS(Min)
		OP(Mean, 6)
		OP(Mean, 8)
		OP(Mean, 13)
		OP(Sum, 6)
		OP(Sum, 8)
		OP(Sum, 13)
#undef OP

		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
