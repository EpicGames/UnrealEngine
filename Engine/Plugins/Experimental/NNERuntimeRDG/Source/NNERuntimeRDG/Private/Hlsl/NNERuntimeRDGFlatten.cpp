// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGFlatten.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorFlatten, TEXT("NNE.Operator.Hlsl.Flatten"));

	/**
	 * Flatten operator implementation
	 */
	class FFlatten : public FOperatorHlsl
	{
	public:

		FFlatten() {}
		virtual ~FFlatten() = default;

		int32 Axis = 1;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			TArray<uint32> OutputShapeData;
			uint32 InnerDimSize = 1;

			for (int32 i = 0; i < Axis; ++i)
			{
				InnerDimSize *= X.GetShape().GetData()[i];
			}
			OutputShapeData.Add(InnerDimSize);
			OutputShapeData.Add(X.GetShape().Volume() / InnerDimSize);

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);

			OutputTensors[0]->SetShape(OutputShape);
			if (X.HasPreparedData())
			{
				OutputTensors[0]->SetPreparedData<uint8>(X.GetPreparedData<uint8>());
			}
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			int32 InputRank = InputTensorDescs[0].GetShape().Rank();
			Axis = Attributes.GetValueOrDefault<int32>(TEXT("axis"), 1);

			if (Axis > InputRank || Axis < -InputRank)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Flatten: 'Axis' attribute should be in the range [-r,r] with r being the rank of the input (name: %ls) however axis is %d while rank is %d.", *InputTensorDescs[0].GetName(), Axis, InputRank);
				return false;
			}
			if (Axis < 0)
			{
				Axis = InputRank + Axis;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorFlatten, "NNE.Operator.Hlsl.Flatten");

			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Data.GetBuffer());
		}
	};

	bool ValidateFlattenOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputFlattens)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateFlattenOperator()
	{
		return new FFlatten();
	}

	bool RegisterFlattenOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Flatten"), TEXT("Onnx")}, 1}, CreateFlattenOperator, ValidateFlattenOperator);
		Registry.OpAdd({{TEXT("Flatten"), TEXT("Onnx")}, 9}, CreateFlattenOperator, ValidateFlattenOperator);
		Registry.OpAdd({{TEXT("Flatten"), TEXT("Onnx")}, 11}, CreateFlattenOperator, ValidateFlattenOperator);
		Registry.OpAdd({{TEXT("Flatten"), TEXT("Onnx")}, 13}, CreateFlattenOperator, ValidateFlattenOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
