// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGGather.h"

#include "Helper/NNERuntimeRDGHelperGather.h"
#include "NNEHlslShadersGatherCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorGather, TEXT("NNE.Operator.Hlsl.Gather"));

	/**
	 * Gather operator implementation
	 */
	template <typename DataElementType, typename IndicesElementType>
	class FGather : public FOperatorHlsl
	{
	public:

		FGather() = default;
		virtual ~FGather() = default;

	private:

		int32 Axis = 0;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2)
			check(OutputTensors.Num() == 1)

			const FTensor& DataTensor = *InputTensors[0];
			const FTensor& IndicesTensor = *InputTensors[1];
			const NNE::FTensorShape& DataShape = DataTensor.GetShape();
			const NNE::FTensorShape& IndicesShape = IndicesTensor.GetShape();

			const int32 OutputRank = IndicesShape.Rank() + DataShape.Rank() - 1;
			
			TArray<uint32> OutputShapeData;
			OutputShapeData.Reserve(OutputRank);

			int32 DataRankIdx = 0;
			while (DataRankIdx < Axis)
			{
				OutputShapeData.Add(DataShape.GetData()[DataRankIdx++]);
			}

			OutputShapeData.Append(IndicesShape.GetData());
			DataRankIdx++;

			while (DataRankIdx < DataShape.Rank())
			{
				OutputShapeData.Add(DataShape.GetData()[DataRankIdx++]);
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			check(OutputShape.Rank() == OutputRank);

			OutputTensors[0]->SetShape(OutputShape);

			CPUHelper::Gather::Apply(DataTensor, IndicesTensor, Axis, *OutputTensors[0]);

			return 0;
		};
		
		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2)
			check(OutputTensorDescs.Num() == 1)

			const NNE::FTensorDesc& Data = InputTensorDescs[0];
			const NNE::FTensorDesc& Indices = InputTensorDescs[1];
			const NNE::FTensorDesc& Output = OutputTensorDescs[0];

			Axis = Attributes.GetValueOrDefault(TEXT("axis"), Axis);
			if (Axis >= Data.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gather: Axis attribute should be inferior to first input rank");
				return false;
			}
			if (Axis < -Data.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gather: Axis attribute should be superior or equal to minus the first input rank");
				return false;
			}
			Axis = Axis >= 0 ? Axis : Data.GetShape().Rank() + Axis;

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 2)
			check(OutputTensors.Num() == 1)
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);
			check(OutputTensors[0]->GetShape().Rank() <= FGatherConstants::MAX_NUM_DIMENSIONS)
			check(InputTensors[0]->GetShape().Rank() > 0)
			check(InputTensors[1]->GetShape().Rank() + (InputTensors[0]->GetShape().Rank() - 1) <= FGatherConstants::MAX_NUM_DIMENSIONS)

			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Indices = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];

			// Set parameters
			TGatherCS::FParameters* Parameters = GraphBuilder.AllocParameters<TGatherCS::FParameters>();
			TGatherCS::FillInParameters(Axis, Data.GetShape(), Indices.GetShape(), *Parameters);
			Parameters->Data = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Data.GetBuffer(), PF_R32_FLOAT));
			// NOTE: Indices tensor can be int64, but UE lacks support of int64 buffers. Here we use a 32-bit pixel format and 
			// in the shader we discard the upper 32-bits of each value. This means that index values need to be representable by 32 bits
			Parameters->Indices = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Indices.GetBuffer(), PF_R32_FLOAT));
			Parameters->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			TGatherCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TGatherCS::FGatherNumOutputDimensions>(Output.GetShape().Rank());
			PermutationVector.Set<TGatherCS::FGather64BitIndices>(Indices.GetDataType() == ENNETensorDataType::Int64);
			TShaderMapRef<TGatherCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FIntVector ThreadGroupCount = TGatherCS::GetGroupCount(*Parameters);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorGather, "NNE.Operator.Hlsl.Gather");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Gather.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Parameters,
				ThreadGroupCount);
		}
	};

	bool ValidateGatherOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
	
		InputValidator.AddSupportedType(ENNETensorDataType::Float, 0);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32, 0);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 0);
		InputValidator.AddRequired(0);

		InputValidator.AddSupportedType(ENNETensorDataType::Int32, 1);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 1);
		InputValidator.AddRequired(1);
		bIsValid &= InputValidator.Validate(InputTypes);

		if(!bIsValid)
		{
			return false;
		}

		if(InputShapes[0].Rank() < 1)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gather: input tensor must have rank >= 1.");
			return false;
		}

		const int32 OutputRank = InputShapes[1].Rank() + (InputShapes[0].Rank() - 1);
		if(OutputRank > NNEHlslShaders::Internal::FGatherConstants::MAX_NUM_DIMENSIONS)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gather: output tensor has rank %d higher than maximum supported: %d.", OutputRank, NNEHlslShaders::Internal::FGatherConstants::MAX_NUM_DIMENSIONS);
			return false;
		}

		return true;
	}

	FOperatorHlsl* CreateGatherOperator()
	{
		return new FGather<float, int32>();
	}

	bool RegisterGatherOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Gather"), TEXT("Onnx")}, 1}, CreateGatherOperator, ValidateGatherOperator);
		Registry.OpAdd({{TEXT("Gather"), TEXT("Onnx")}, 11}, CreateGatherOperator, ValidateGatherOperator);
		Registry.OpAdd({{TEXT("Gather"), TEXT("Onnx")}, 13}, CreateGatherOperator, ValidateGatherOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl