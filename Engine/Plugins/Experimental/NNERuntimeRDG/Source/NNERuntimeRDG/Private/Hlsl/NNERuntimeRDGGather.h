// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGBase.h"
#include "NNERuntimeRDGHlslOp.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	bool RegisterGatherOperator(FOperatorRegistryHlsl& Registry);
} // UE::NNERuntimeRDG::Private::Hlsl
