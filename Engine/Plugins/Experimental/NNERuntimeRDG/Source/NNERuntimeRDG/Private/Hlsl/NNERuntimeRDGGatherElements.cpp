// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGGatherElements.h"

#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNEHlslShadersGatherElementsCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorGatherElements, TEXT("NNE.Operator.Hlsl.GatherElements"));

	/**
	 * GatherElements operator implementation
	 */
	template<int Version>
	class FGatherElements : public FOperatorHlsl
	{
	public:

		FGatherElements() {}
		virtual ~FGatherElements() = default;

		int32 Rank;
		int32 Axis;
	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);

			OutputTensors[0]->SetShape(InputTensors[1]->GetShape());

			return 0;
		}

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);

			Rank = InputTensorDescs[0].GetShape().Rank();
			Axis = Attributes.GetValueOrDefault<int32>(TEXT("axis"), 0);
			if (Axis < -Rank || Axis >= Rank)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "GatherElement: 'axis' attribute needs to be in the range [-Rank, Rank - 1], but it is %i with a rank of %i.", Axis, Rank);
				return false;
			}
			if (Axis < 0)
			{
				Axis += Rank;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;
			
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Indices = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];

			FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			// NOTE: Indices tensor can be int64, but UE lacks support of int64 buffers. Here we use a 32-bit pixel format and 
			// in the shader we discard the upper 32-bits of each value. This means that index values need to be representable by 32 bits
			FRDGBufferSRVRef IndicesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Indices.GetBuffer(), PF_R32_SINT));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FGatherElementsConstants::NUM_GROUP_THREADS);

			// Set parameters
			FGatherElementsCS::FParameters* Params = GraphBuilder.AllocParameters<FGatherElementsCS::FParameters>();
			Params->Input = InputSRV;
			Params->Indices = IndicesSRV;
			Params->Output = OutputUAV;
			Params->Axis = Axis;
			Params->OutputSize = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FGatherElementsConstants::NUM_GROUP_THREADS;

			TConstArrayView<uint32> InputShape = Input.GetShape().GetData();
			TConstArrayView<uint32> OutputShape = Output.GetShape().GetData();
			int32 InputStride = 1;
			int32 OutputStride = 1;
			for (int i = Rank - 1; i >= 0; --i)
			{
				Params->OneDivOutputStrides[i].X = 1.0 / (float)OutputStride;
				Params->Input_OutputStrides[i].X = InputStride;
				Params->Input_OutputStrides[i].Y = OutputStride;
				OutputStride *= OutputShape[i];
				InputStride *= InputShape[i];
			}
			Params->AxisSize = InputShape[Axis];

			FGatherElementsCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FGatherElementsCS::FGatherElementsDimensions>(Rank);
			PermutationVector.Set<FGatherElementsCS::FGatherElements64BitIndices>(Indices.GetDataType() == ENNETensorDataType::Int64);

			TShaderMapRef<FGatherElementsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorGatherElements, "NNE.Operator.Hlsl.GatherElements");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.GatherElements.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	template<int Version>
	bool ValidateGatherElementsOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);
		
		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 1);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32, /*TemplateIdx*/ 1);
		InputValidator.AddRequired();
		InputValidator.AddRequired(1);
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<int Version>
	FOperatorHlsl* CreateGatherElementsOperator()
	{
		return new FGatherElements<Version>();
	}

	bool RegisterGatherElementsOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		#define OP(Version) \
		Registry.OpAdd({{TEXT("GatherElements"), TEXT("Onnx")}, Version}, CreateGatherElementsOperator<Version>, ValidateGatherElementsOperator<Version>);

		OP(11)
		OP(13)

		#undef OP
		return true;
	}

} // UE::NNERuntimeRDG::Private::Hlsl
