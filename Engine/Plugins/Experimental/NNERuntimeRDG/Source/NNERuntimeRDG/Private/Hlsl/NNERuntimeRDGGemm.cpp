// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGGemm.h"

#include "NNEHlslShadersGemmCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorGemm, TEXT("NNE.Operator.Hlsl.Gemm"));

	/**
	 * Gemm operator implementation
	 */
	class TGemm : public FOperatorHlsl
	{
	public:

		TGemm() {}
		virtual ~TGemm() = default;

	private:

		float InputAlpha = 1.0f;
		float InputBeta = 1.0f;
		int32 InputTransA = 0;
		int32 InputTransB = 0;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);

			const NNE::FTensorShape& InputA = InputTensors[0]->GetShape();
			const NNE::FTensorShape& InputB = InputTensors[1]->GetShape();
			if (InputA.Rank() != 2 || InputB.Rank() != 2)
			{
				return -1;
			}

			uint32 M = InputTransA != 0 ? InputA.GetData()[1] : InputA.GetData()[0];
			uint32 N = InputTransB != 0 ? InputB.GetData()[0] : InputB.GetData()[1];
			TArray<uint32> OutputShapeData;

			OutputShapeData.Emplace(M);
			OutputShapeData.Emplace(N);
			
			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			
			OutputTensors[0]->SetShape(OutputShape);
			return 0;
		};
		
		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() >= 2 && InputTensorDescs.Num() <= 3);
			check(OutputTensorDescs.Num() == 1);

            const NNE::FTensorDesc& InputA = InputTensorDescs[0];
			const NNE::FTensorDesc& InputB = InputTensorDescs[1];

			if (InputA.GetShape().Rank() != 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gemm: First input should be of rank 2");
				return false;
			}
			if (InputB.GetShape().Rank() != 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gemm: Second input should be of rank 2");
				return false;
			}
			if (InputTensorDescs.Num() == 3)
			{
				const NNE::FTensorDesc& InputC = InputTensorDescs[2];
				if (InputC.GetShape().Rank() > 2)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Gemm: Third input should be of rank 2 or less");
					return false;
				}
			}

			InputAlpha = Attributes.GetValueOrDefault(TEXT("alpha"), InputAlpha);
			InputBeta = Attributes.GetValueOrDefault(TEXT("beta"), InputBeta);
			InputTransA = Attributes.GetValueOrDefault(TEXT("transA"), InputTransA);
			InputTransB = Attributes.GetValueOrDefault(TEXT("transB"), InputTransB);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);

			const EGemmAlgorithm Algorithm = EGemmAlgorithm::Simple32x32;
			const FTensorRDG& InputA = *InputTensors[0];
			const FTensorRDG& InputB = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];
			const FTensorRDG* InputC = nullptr;
			float CConstantScalar = 0.0f;
			EGemmCScalar CScalarMode = EGemmCScalar::MAX;

			if (InputTensors.Num() == 3)
			{
				check(InputTensors[2] != nullptr);
				if (InputTensors[2]->HasPreparedData() && InputTensors[2]->GetVolume() == 1)
				{
					CConstantScalar = InputTensors[2]->GetPreparedData<float>()[0];
					CScalarMode = EGemmCScalar::Yes;
				}
				else
				{
					InputC = InputTensors[2];
					CScalarMode = EGemmCScalar::No;
				}
			}
			else
			{
				CScalarMode = EGemmCScalar::NoBias;
			}
			check(CScalarMode != EGemmCScalar::MAX);
			
			// Set parameters
			const NNE::FTensorShape* InputCShape = InputC != nullptr ? &InputC->GetShape() : nullptr;
			TGemmCS::FParameters* Parameters = GraphBuilder.AllocParameters<TGemmCS::FParameters>();
			TGemmCS::FillInParameters(InputAlpha, InputBeta, InputTransA, InputTransB, InputA.GetShape(), InputB.GetShape(), InputCShape, CConstantScalar, *Parameters);
			Parameters->A = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputA.GetBuffer(), PF_R32_FLOAT));
			Parameters->B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputB.GetBuffer(), PF_R32_FLOAT));
			if (InputC != nullptr) {
				Parameters->C = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputC->GetBuffer(), PF_R32_FLOAT));
			}
			Parameters->Y = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			TGemmCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TGemmCS::FGemmCScalar>(CScalarMode);
			PermutationVector.Set<TGemmCS::FGemmAlgorithm>(Algorithm);
			PermutationVector.Set<TGemmCS::FGemmNumStackDimensions>(0);
			TShaderMapRef<TGemmCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FIntVector ThreadGroupCount = TGemmCS::GetGroupCount(*Parameters, Algorithm, 0);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorGemm, "NNE.Operator.Hlsl.Gemm");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Parameters,
				ThreadGroupCount);
		}
	};

	bool ValidateGemmOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		//This match version 9, 11, 13 of the Gemm operator see
		//https://onnx.ai/onnx/operators/onnx__Gemm.html#l-onnx-doc-gemm
		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("alpha"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("beta"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("transA"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("transB"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddOptional();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateGemmOperator()
	{
		return new TGemm();
	}

	bool RegisterGemmOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Gemm"), TEXT("Onnx")}, 7}, CreateGemmOperator, ValidateGemmOperator);
		Registry.OpAdd({{TEXT("Gemm"), TEXT("Onnx")}, 9}, CreateGemmOperator, ValidateGemmOperator);
		Registry.OpAdd({{TEXT("Gemm"), TEXT("Onnx")}, 11}, CreateGemmOperator, ValidateGemmOperator);
		Registry.OpAdd({{TEXT("Gemm"), TEXT("Onnx")}, 13}, CreateGemmOperator, ValidateGemmOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
