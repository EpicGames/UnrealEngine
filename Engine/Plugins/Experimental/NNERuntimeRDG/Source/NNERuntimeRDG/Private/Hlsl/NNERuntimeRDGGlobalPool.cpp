// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGGlobalPool.h"

#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersReduceCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorGlobalPool, TEXT("NNE.Operator.Hlsl.GlobalPool"));

	/**
	 * GlobalPool operator implementation
	 */
	class FGlobalPool : public FOperatorHlsl
	{

	public:

		FGlobalPool(UE::NNEHlslShaders::Internal::EReduceOperatorType InReduceOperatorType):ReduceOperatorType(InReduceOperatorType) {};
		virtual ~FGlobalPool() = default;

	private:

		const UE::NNEHlslShaders::Internal::EReduceOperatorType ReduceOperatorType;
		constexpr static int32 FirstReducedDimension = 2;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const NNE::FTensorShape& InputShape = InputTensors[0]->GetShape();
			const int32 InputRank = InputShape.Rank();

			check(InputRank > FirstReducedDimension);

			TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> OutputShape(InputShape.GetData());

			for (int32 Axis = FirstReducedDimension; Axis < InputRank; ++Axis)
			{
				OutputShape[Axis] = 1;
			}
			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			if (InputTensorDescs[0].GetShape().Rank() != OutputTensorDescs[0].GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "GlobalPool: Operators requires the output to have the same rank as the input.");
				return false;
			}

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();

			if (InputRank <= FirstReducedDimension)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "GlobalPool: Operators requires input tensor to be at least 3-D (but got rank %d)", InputRank);
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != FTensorRDGRef{});
			check(OutputTensors[0] != FTensorRDGRef{});

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorGlobalPool, "NNE.Operator.Hlsl.GlobalPool");

			TReduceCS::FParameters* Parameters = GraphBuilder.AllocParameters<TReduceCS::FParameters>();
			TReduceCS::FillInParameters(Input.GetShape().GetData(), FirstReducedDimension, Parameters);
			Parameters->AxisSize *= Parameters->NumElemAfterAxis;// GlobalPool reduce all trailing dimensions thus we can flatten them.
			Parameters->NumElemAfterAxis = 1;
			
			TReduceCS::EnqueueRDG(GraphBuilder, Parameters, Input.GetBuffer(), Output.GetBuffer(), ReduceOperatorType);
		}
	};

	bool ValidateGlobalPoolOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		// This match version 1 of the GlobalAveragePool and GlobalMaxPool operators
		// https://github.com/onnx/onnx/blob/main/docs/Changelog.md#GlobalPool-1
		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template< UE::NNEHlslShaders::Internal::EReduceOperatorType ReduceOperatorType >
	FOperatorHlsl* CreateGlobalPoolOperator()
	{
		return new FGlobalPool(ReduceOperatorType);
	}

	bool RegisterGlobalPoolOperators(FOperatorRegistryHlsl& Registry)
	{
		Registry.OpAdd({{TEXT("GlobalAveragePool"), TEXT("Onnx")}, 1}, CreateGlobalPoolOperator<UE::NNEHlslShaders::Internal::EReduceOperatorType::Average>, ValidateGlobalPoolOperator);
		Registry.OpAdd({{TEXT("GlobalMaxPool"), TEXT("Onnx")}, 1}, CreateGlobalPoolOperator<UE::NNEHlslShaders::Internal::EReduceOperatorType::Max>, ValidateGlobalPoolOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
