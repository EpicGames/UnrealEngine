// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGIdentity.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorIdentity, TEXT("NNE.Operator.Hlsl.Identity"));

	/**
	 * Identity operator implementation
	 */
	class FIdentity : public FOperatorHlsl
	{
	public:

		FIdentity() {}
		virtual ~FIdentity() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];

			OutputTensors[0]->SetShape(X.GetShape());
			if (X.HasPreparedData())
			{
				OutputTensors[0]->SetPreparedData<uint8>(X.GetPreparedData<uint8>());
			}
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorIdentity, "NNE.Operator.Hlsl.Identity");

			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Data.GetBuffer());
		}
	};

	bool ValidateIdentityOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputIdentitys)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateIdentityOperator()
	{
		return new FIdentity();
	}

	bool RegisterIdentityOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Identity"), TEXT("Onnx")}, 1}, CreateIdentityOperator, ValidateIdentityOperator);
		Registry.OpAdd({{TEXT("Identity"), TEXT("Onnx")}, 13}, CreateIdentityOperator, ValidateIdentityOperator);
		Registry.OpAdd({{TEXT("Identity"), TEXT("Onnx")}, 14}, CreateIdentityOperator, ValidateIdentityOperator);
		Registry.OpAdd({{TEXT("Identity"), TEXT("Onnx")}, 16}, CreateIdentityOperator, ValidateIdentityOperator);
		Registry.OpAdd({{TEXT("Identity"), TEXT("Onnx")}, 19}, CreateIdentityOperator, ValidateIdentityOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
