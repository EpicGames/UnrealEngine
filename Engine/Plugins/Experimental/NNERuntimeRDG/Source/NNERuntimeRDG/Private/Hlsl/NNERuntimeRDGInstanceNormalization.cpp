// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGInstanceNormalization.h"

#include "NNEHlslShadersInstanceNormalizationCS.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersReduceCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace
{

int ValidateInput(const UE::NNE::FTensorShape& Input, const UE::NNE::FTensorShape& Scale, const UE::NNE::FTensorShape& Bias)
{
	if (Scale.Rank() != 1)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization: Input scale should be of rank 1: %d", Scale.Rank());
		return -1;
	}
	if (Scale.GetData()[0] != Input.GetData()[1])
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization: Input scale size should be equal to channel count: %d vs %d", Scale.GetData()[0], Input.GetData()[1]);
		return -1;
	}

	if (Bias.Rank() != 1)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization: Input B should be of rank 1: %d", Bias.Rank());
		return -1;
	}
	if (Bias.GetData()[0] != Input.GetData()[1])
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization: Input B size should be equal to channel count: : %d vs %d", Bias.GetData()[0], Input.GetData()[1]);
		return -1;
	}

	return 0;
}

}

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorInstanceNormalization, TEXT("NNE.Operator.Hlsl.InstanceNormalization"));

	/**
	 * InstanceNormalization operator implementation
	 */
	class TInstanceNormalization : public FOperatorHlsl
	{
	public:

		TInstanceNormalization() {}
		virtual ~TInstanceNormalization() = default;

	private:

		float Epsilon = 1e-5f;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0]->GetShape().Rank() > 2);

			if (const int Res = ValidateInput(InputTensors[0]->GetShape(), InputTensors[1]->GetShape(), InputTensors[2]->GetShape()); Res < 0)
			{
				return Res;
			}

			OutputTensors[0]->SetShape(InputTensors[0]->GetShape());

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensorDescs.Num() == 3);
			check(OutputTensorDescs.Num() == 1);

			Epsilon = Attributes.GetValueOrDefault<float>(TEXT("epsilon"), 1e-5f);

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();

			if (InputRank < 3)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization input data should be at least of rank 3 but got: %d", InputRank);
				return false;
			}

			if (InputRank != OutputTensorDescs[0].GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "InstanceNormalization requires the output to have the same rank as the input.");
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(InputTensors[2] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Scale = *InputTensors[1];
			const FTensorRDG& Bias = *InputTensors[2];
			const FTensorRDG& Output = *OutputTensors[0];
			const NNE::FTensorShape& InputShape = Input.GetShape();
			constexpr int32 ReductionAxis = 2;
			check(InputShape.Rank() >= ReductionAxis);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorInstanceNormalization, "NNE.Operator.Hlsl.InstanceNormalization");

			// First apply Reduction() to temp buffers getting Mean and InvStdDev
			TReduceCS::FParameters* ReduceParameters = GraphBuilder.AllocParameters<TReduceCS::FParameters>();
			TReduceCS::FillInParameters(InputShape.GetData(), ReductionAxis, ReduceParameters);
			ReduceParameters->AxisSize *= ReduceParameters->NumElemAfterAxis;// InstanceNormalization flatten the input tensor to a 2D one
			ReduceParameters->NumElemAfterAxis = 1;
			ReduceParameters->Epsilon = Epsilon;
			const FRDGBufferDesc InstNormTempBufferDesc = FRDGBufferDesc::CreateBufferDesc(Output.GetElementByteSize(), ReduceParameters->NumElemBeforeAxis);

			FRDGBufferRef MeanBuffer = GraphBuilder.CreateBuffer(InstNormTempBufferDesc, TEXT("NNE.Operator.Hlsl.InstanceNormalization.TempMeanBuffer"), ERDGBufferFlags::None);
			FRDGBufferRef InvStdDevBuffer = GraphBuilder.CreateBuffer(InstNormTempBufferDesc, TEXT("NNE.Operator.Hlsl.InstanceNormalization.TempInvStdDevBuffer"), ERDGBufferFlags::None);

			TReduceCS::EnqueueRDG(GraphBuilder, ReduceParameters, Input.GetBuffer(), MeanBuffer, EReduceOperatorType::AverageInvStdDev, InvStdDevBuffer);

			//Then InstanceNormalization
			const int32 NumElements = Input.GetVolume();
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(NumElements, FInstanceNormalizationConstants::NUM_GROUP_THREADS);

			TInstanceNormalizationCS::FParameters* InstNormParameters = GraphBuilder.AllocParameters<TInstanceNormalizationCS::FParameters>();
			InstNormParameters->InstanceSize = ReduceParameters->AxisSize;
			InstNormParameters->ChannelSize = InputShape.GetData()[1];
			InstNormParameters->Num = NumElements;
			InstNormParameters->ThreadCountX = ThreadGroupCount.X * FInstanceNormalizationConstants::NUM_GROUP_THREADS;
			InstNormParameters->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			InstNormParameters->InputScale = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Scale.GetBuffer(), PF_R32_FLOAT));
			InstNormParameters->InputBias = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias.GetBuffer(), PF_R32_FLOAT));
			InstNormParameters->InputMean = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(MeanBuffer, PF_R32_FLOAT));
			InstNormParameters->InputInvStdDev = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InvStdDevBuffer, PF_R32_FLOAT));
			InstNormParameters->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			TShaderMapRef<TInstanceNormalizationCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.InstanceNormalization.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				InstNormParameters,
				ThreadGroupCount);
		}
	};

	bool ValidateInstanceNormalizationOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("epsilon"), ENNERuntimeRDGDataAttributeDataType::Float);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateInstanceNormalizationOperator()
	{
		return new TInstanceNormalization();
	}

	bool RegisterInstanceNormalizationOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("InstanceNormalization"), TEXT("Onnx")}, 6}, CreateInstanceNormalizationOperator, ValidateInstanceNormalizationOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
