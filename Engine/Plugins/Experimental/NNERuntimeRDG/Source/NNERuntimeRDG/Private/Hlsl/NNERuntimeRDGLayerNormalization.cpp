// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGLayerNormalization.h"

#include "NNEHlslShadersLayerNormalizationCS.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersReduceCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorLayerNormalization, TEXT("NNE.Operator.Hlsl.LayerNormalization"));

	/**
	 * LayerNormalization operator implementation
	 */
	class TLayerNormalization : public FOperatorHlsl
	{
	public:

		TLayerNormalization() {}
		virtual ~TLayerNormalization() = default;

	private:

		float Epsilon = 1e-5f;
		int32 Axis = -1;
		int32 StashType = 1;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() >= 1 && OutputTensors.Num() <= 3);

			auto ValidateScaleOrBias = [Axis = Axis] (const UE::NNE::FTensorShape& Input, const UE::NNE::FTensorShape& ScaleOrBias) -> int
			{
				if (ScaleOrBias.Rank() > (Input.Rank() - Axis))
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "LayerNormalization: Scale/bias tensor rank is invalid: %d", ScaleOrBias.Rank());
					return -1;
				}
				for(int DimIdx = 0; DimIdx < ScaleOrBias.Rank(); ++DimIdx)
				{
					if(Input.GetData()[DimIdx + Axis] != ScaleOrBias.GetData()[DimIdx] && ScaleOrBias.GetData()[DimIdx] != 1)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "LayerNormalization: Scale/bias tensor shape not equal nor broadcastable to input's");
						return -1;
					}
				}

				return 0;
			};

			if (const int Res = ValidateScaleOrBias(InputTensors[0]->GetShape(), InputTensors[1]->GetShape()); Res < 0)
			{
				return Res;
			}
			if (InputTensors.Num() == 3)
			{
				if (const int Res = ValidateScaleOrBias(InputTensors[0]->GetShape(), InputTensors[2]->GetShape()); Res < 0)
				{
					return Res;
				}
			}

			OutputTensors[0]->SetShape(InputTensors[0]->GetShape());

			auto GetMeanOrInvStdDevShape = [Axis = Axis] (const UE::NNE::FTensorShape& Input) -> UE::NNE::FTensorShape
			{
				TArray<uint32> OutShape;
				OutShape.SetNumUninitialized(Input.Rank());
				
				for(int DimIdx = 0; DimIdx < Axis; ++DimIdx)
				{
					OutShape[DimIdx] = Input.GetData()[DimIdx];
				}
				for(int DimIdx = Axis; DimIdx < Input.Rank(); ++DimIdx)
				{
					OutShape[DimIdx] = 1;
				}

				return UE::NNE::FTensorShape::Make(OutShape);
			};

			if (OutputTensors.Num() >= 2)
			{
				OutputTensors[1]->SetShape(GetMeanOrInvStdDevShape(InputTensors[0]->GetShape()));
			}
			if (OutputTensors.Num() >= 3)
			{
				OutputTensors[2]->SetShape(GetMeanOrInvStdDevShape(InputTensors[0]->GetShape()));
			}

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() >= 2 && InputTensorDescs.Num() <= 3);
			check(OutputTensorDescs.Num() >= 1 && OutputTensorDescs.Num() <= 3);

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();

			Epsilon = Attributes.GetValueOrDefault<float>(TEXT("epsilon"), Epsilon);
			Axis = Attributes.GetValueOrDefault<int32>(TEXT("axis"), Axis);
			if(Axis < 0)
			{
				Axis += InputRank;
			}
			StashType = Attributes.GetValueOrDefault<int32>(TEXT("stash_type"), StashType);

			if (InputRank != OutputTensorDescs[0].GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "LayerNormalization: Output should have the same rank as the input.");
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 2 && InputTensors.Num() <= 3);
			check(OutputTensors.Num() >= 1 && OutputTensors.Num() <= 3);

			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			if(InputTensors.Num() == 3)
			{
				check(InputTensors[2] != nullptr);
			}

			check(OutputTensors[0] != nullptr);
			if(OutputTensors.Num() >= 2)
			{
				check(OutputTensors[1] != nullptr);
			}
			if(OutputTensors.Num() >= 3)
			{
				check(OutputTensors[2] != nullptr);
			}

			bool bHasBias = InputTensors.Num() == 3;
			bool bWriteMean = OutputTensors.Num() >= 2;
			bool bWriteInvStdDev = OutputTensors.Num() >= 3;

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorLayerNormalization, "NNE.Operator.Hlsl.LayerNormalization");

			const FTensorRDG& Input = *InputTensors[0];
			const NNE::FTensorShape& InputShape = Input.GetShape();
			
			// First apply Reduction() to temp buffers getting Mean and InvStdDev
			TReduceCS::FParameters* ReduceParameters = GraphBuilder.AllocParameters<TReduceCS::FParameters>();
			// Need to feed a shape such that the last dimension corresponds to the slice to reduce
			TArray<uint32> ReductionShape;
			ReductionShape.Init(1, Axis + 1);
			for(int Idx = 0; Idx < InputShape.Rank(); ++Idx)
			{
				if(Idx < Axis)
				{
					ReductionShape[Idx] = InputShape.GetData()[Idx];
				}
				else
				{
					ReductionShape[Axis] *= InputShape.GetData()[Idx];
				}
			}

			TReduceCS::FillInParameters(ReductionShape, Axis, ReduceParameters);
			ReduceParameters->Epsilon = Epsilon;
			// NOTE: once have support for more datatypes, make this depend on stash_type attribute
			uint32 BytesPerElementTemp = Input.GetElementByteSize(); 
			const FRDGBufferDesc LayerNormTempBufferDesc = FRDGBufferDesc::CreateBufferDesc(BytesPerElementTemp, ReduceParameters->NumElemBeforeAxis);

			FRDGBufferRef MeanBuffer;
			if(bWriteMean)
			{
				const FTensorRDG& OutputMean = *OutputTensors[1];
				MeanBuffer = OutputMean.GetBuffer();
			}
			else
			{
				MeanBuffer = GraphBuilder.CreateBuffer(LayerNormTempBufferDesc, TEXT("NNE.Operator.Hlsl.LayerNormalization.TempMeanBuffer"), ERDGBufferFlags::None);
			}

			FRDGBufferRef InvStdDevBuffer;
			if(bWriteInvStdDev)
			{
				const FTensorRDG& OutputInvStdDev = *OutputTensors[2];
				InvStdDevBuffer = OutputInvStdDev.GetBuffer();
			}
			else
			{
				InvStdDevBuffer = GraphBuilder.CreateBuffer(LayerNormTempBufferDesc, TEXT("NNE.Operator.Hlsl.LayerNormalization.TempInvStdDevBuffer"), ERDGBufferFlags::None);
			}

			TReduceCS::EnqueueRDG(GraphBuilder, ReduceParameters, Input.GetBuffer(), MeanBuffer, EReduceOperatorType::AverageInvStdDev, InvStdDevBuffer);

			// Then LayerNormalization
			TLayerNormalizationCS::FParameters* LayerNormParameters = GraphBuilder.AllocParameters<TLayerNormalizationCS::FParameters>();
			TLayerNormalizationCS::FillInParameters(InputShape.GetData(), Axis, Epsilon, LayerNormParameters);

			const uint32 NumElements = Input.GetVolume();
			LayerNormParameters->Num = NumElements;
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(NumElements, FLayerNormalizationConstants::NUM_GROUP_THREADS);;
			LayerNormParameters->ThreadCountX = ThreadGroupCount.X * FLayerNormalizationConstants::NUM_GROUP_THREADS;

			FillTensorStrideShaderParameters(Input, LayerNormParameters->InputTensorInfo, /*Idx*/ 0);
			LayerNormParameters->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));

			const FTensorRDG& Scale = *InputTensors[1];
			FillTensorStrideShaderParameters(Scale, LayerNormParameters->ScaleTensorInfo, /*Idx*/ 0, /*TargetNumDimsForBroadcast*/ InputShape.Rank() - Axis);
			LayerNormParameters->InputScale = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Scale.GetBuffer(), PF_R32_FLOAT));

			if(bHasBias)
			{
				const FTensorRDG& Bias = *InputTensors[2];
				FillTensorStrideShaderParameters(Bias, LayerNormParameters->BiasTensorInfo, /*Idx*/ 0, /*TargetNumDimsForBroadcast*/ InputShape.Rank() - Axis);
				LayerNormParameters->InputBias = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Bias.GetBuffer(), PF_R32_FLOAT));
			}

			LayerNormParameters->InputMean = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(MeanBuffer, PF_R32_FLOAT));
			LayerNormParameters->InputInvStdDev = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InvStdDevBuffer, PF_R32_FLOAT));

			const FTensorRDG& Output = *OutputTensors[0];
			LayerNormParameters->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			TLayerNormalizationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TLayerNormalizationCS::FLayerNormalizationNumDimensions>(InputShape.Rank());
			PermutationVector.Set<TLayerNormalizationCS::FLayerNormalizationHasB>(bHasBias);

			TShaderMapRef<TLayerNormalizationCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.LayerNormalization.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				LayerNormParameters,
				ThreadGroupCount);
		}
	};

	bool ValidateLayerNormalizationOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("epsilon"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("stash_type"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		InputValidator.AddOptional();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateLayerNormalizationOperator()
	{
		return new TLayerNormalization();
	}

	bool RegisterLayerNormalizationOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("LayerNormalization"), TEXT("Onnx")}, 17}, CreateLayerNormalizationOperator, ValidateLayerNormalizationOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
