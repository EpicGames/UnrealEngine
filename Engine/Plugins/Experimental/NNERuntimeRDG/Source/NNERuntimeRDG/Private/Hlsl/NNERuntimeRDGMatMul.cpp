// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGMatMul.h"

#include "NNEHlslShadersGemmCS.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
    DECLARE_GPU_STAT_NAMED(FNNEOperatorMatMul, TEXT("NNE.Operator.Hlsl.MatMul"));

	/**
	 * MatMul operator implementation
	 */
	class TMatMul : public FOperatorHlsl
	{
	public:

		TMatMul() {}
		virtual ~TMatMul() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);

			const NNE::FTensorShape& InputA = InputTensors[0]->GetShape();
			const NNE::FTensorShape& InputB = InputTensors[1]->GetShape();

			if (InputA.Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Matmul: First input should be at least of rank 2");
				return -1;
			}
			if (InputB.Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Matmul: Second input should be at least of rank 2");
				return -1;
			}
			if (InputA.GetData()[InputA.Rank() - 1] != InputB.GetData()[InputB.Rank() - 2])
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Matmul: First input last dimension should be equal to second input last dimension");
				return -1;
			}

			const int32 OutputRank = FMath::Max(InputA.Rank(), InputB.Rank());
			TArray<uint32> OutputShapeData;
			OutputShapeData.SetNumUninitialized(OutputRank);
			
			//Broadcast
			for (int32 i = 0; i < OutputRank; ++i)
			{
				int32 AIndex = InputA.Rank() - 1 - i;
				int32 BIndex = InputB.Rank() - 1 - i;
				int32 AValue = AIndex >= 0 ? InputA.GetData()[AIndex] : 1;
				int32 BValue = BIndex >= 0 ? InputB.GetData()[BIndex] : 1;
				int32 OutputValue = FMath::Max(AValue, BValue);
				OutputShapeData[OutputRank - 1 - i] = OutputValue;
			}
			
			//2D Mat
			OutputShapeData[OutputRank - 2] = InputA.GetData()[InputA.Rank() - 2];
			OutputShapeData[OutputRank - 1] = InputB.GetData()[InputB.Rank() - 1];

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);

			OutputTensors[0]->SetShape(OutputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);

			const NNE::FTensorDesc& InputA = InputTensorDescs[0];
			const NNE::FTensorDesc& InputB = InputTensorDescs[1];

			if (InputA.GetShape().Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Matmul: First input should be at least of rank 2");
				return false;
			}
			if (InputB.GetShape().Rank() < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Matmul: Second input should be at least of rank 2");
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& InputA = *InputTensors[0];
			const FTensorRDG& InputB = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];

			const EGemmAlgorithm Algorithm = EGemmAlgorithm::Simple32x32;

			const int32 NumStackDimensions = FMath::Max(FMath::Max(InputA.GetShape().Rank(), InputB.GetShape().Rank()) - 2, 0);

			// Set parameters
			TGemmCS::FParameters* Parameters = GraphBuilder.AllocParameters<TGemmCS::FParameters>();
			TGemmCS::FillInParametersMatMul(InputA.GetShape(), InputB.GetShape(), *Parameters);
			Parameters->A = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputA.GetBuffer(), PF_R32_FLOAT));
			Parameters->B = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(InputB.GetBuffer(), PF_R32_FLOAT));
			Parameters->Y = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			TGemmCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TGemmCS::FGemmCScalar>(EGemmCScalar::NoBias);
			PermutationVector.Set<TGemmCS::FGemmAlgorithm>(Algorithm);
			PermutationVector.Set<TGemmCS::FGemmNumStackDimensions>(NumStackDimensions);
			TShaderMapRef<TGemmCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FIntVector ThreadGroupCount = TGemmCS::GetGroupCount(*Parameters, Algorithm, NumStackDimensions);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorMatMul, "NNE.Operator.Hlsl.MatMul");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.MatMul.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Parameters,
				ThreadGroupCount);
		}
	};

	bool ValidateMatMulOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateMatMulOperator()
	{
		return new TMatMul();
	}

	bool RegisterMatMulOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("MatMul"), TEXT("Onnx")}, 1}, CreateMatMulOperator, ValidateMatMulOperator);
		Registry.OpAdd({{TEXT("MatMul"), TEXT("Onnx")}, 9}, CreateMatMulOperator, ValidateMatMulOperator);
		Registry.OpAdd({{TEXT("MatMul"), TEXT("Onnx")}, 13}, CreateMatMulOperator, ValidateMatMulOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
