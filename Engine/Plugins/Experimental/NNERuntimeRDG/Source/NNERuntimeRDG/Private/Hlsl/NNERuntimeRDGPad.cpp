// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGPad.h"

#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersPadCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorPad, TEXT("NNE.Operator.Hlsl.Pad"));

	using EPadMode = UE::NNEHlslShaders::Internal::EPadMode;

	/**
	 * Pad operator implementation
	 */
	template<int Version>
	class FPad : public FOperatorHlsl
	{
	public:

		FPad() {}
		virtual ~FPad() = default;

		TArray<int32> Pads;
		float Value = 0.0f;
		EPadMode Mode;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			CheckInputTensorCount(InputTensors.Num());
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			const int32 Rank = X.GetShape().Rank();

			if constexpr (Version >= 11)
			{
				const FTensorRef PadsTensor = InputTensors[1];
				if(!PadsTensor->IsConstant())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: Input 'pads' (name: %ls) should be constant.", *PadsTensor->GetName());
					return -1;
				}

				if (InputTensors.Num() >= 3 && !InputTensors[2]->IsEmpty())
				{
					const FTensor& ValueTensor = *InputTensors[2];

					if(!ValueTensor.HasPreparedData())
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: Input 'constant_value' (name: %ls) should be constant.", *ValueTensor.GetName());
						return -1;
					}

					if(ValueTensor.GetPreparedData<float>().Num() != 1)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: Input 'constant_value' (name: %ls) should be scalar, however it is not.", *ValueTensor.GetName());
						return -1;
					}

					Value = ValueTensor.GetPreparedData<float>()[0];
				}

				if (InputTensors.Num() >= 4 && !InputTensors[3]->IsEmpty())
				{
					const FTensorRef AxesTensor = InputTensors[3];

					if(!AxesTensor->IsConstant())
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: Input 'axes' (name: %ls) should be constant.", *AxesTensor->GetName());
						return -1;
					}

					TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Axes;
					TArray<int32, TInlineAllocator<2 * NNE::FTensorShape::MaxRank>> RelativePads;
					if (AxesTensor->HasPreparedData())
					{
						OperatorHelper::GetInt32ArrayFromConstTensor(Axes, AxesTensor);
					}

					if (PadsTensor->HasPreparedData())
					{
						OperatorHelper::GetInt32ArrayFromConstTensor(RelativePads, PadsTensor);
					}

					if (RelativePads.Num() != Axes.Num() * 2)
					{
						UE_LOG(LogNNERuntimeRDGHlsl, Warning, TEXT("Pad: Input 'axes' (name: %s) has to have a size that is twice the size \
													  of pad input 'pads' (name: %s), but they have size %i and %i respectively."), 
												*AxesTensor->GetName(), *PadsTensor->GetName(), Axes.Num(), RelativePads.Num());
						return -1;
					}

					Pads.Init(0, 2 * Rank);
					for (int AxesIndex = 0; AxesIndex < Axes.Num(); ++AxesIndex)
					{
						int32 Axis = Axes[AxesIndex];
						if (Axis < -Rank || Axis >= Rank)
						{
							UE_LOG(LogNNERuntimeRDGHlsl, Warning, TEXT("Pad: Input value at index %i of the 'axes' (name: %s) tensor \
														  needs to be in the range [-Rank, Rank - 1], but value is %i with a rank of %i."), 
													AxesIndex, *AxesTensor->GetName(), Axis, Rank);
							return -1;
						}
						if (Axis < 0)
						{
							Axis = Rank + Axis;
						}
						Pads[Axis] = RelativePads[AxesIndex];
						Pads[Axis + Rank] = RelativePads[AxesIndex + Axes.Num()];
					}
				}
				else
				{
					if (!PadsTensor->HasPreparedData())
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: 'pads' attribute length (%d) should be twice the rank of input X (%d).", 0, X.GetShape().Rank());
						return -1;
					}
					OperatorHelper::GetInt32ArrayFromConstTensor(Pads, PadsTensor);
				}
			}

			if ((2*X.GetShape().Rank()) != Pads.Num())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: 'pads' attribute length (%d) should be twice the rank of input X (%d).", Pads.Num(), X.GetShape().Rank());
				return false;
			}

			TArray<uint32> OutputShapeData;
			for (int32 i = 0; i < X.GetShape().Rank(); ++i)
			{
				int32 PrePad = Pads[i];
				int32 PostPad = Pads[i + X.GetShape().Rank()];
				int32 OutputDim = PrePad + X.GetShape().GetData()[i] + PostPad;
				if (OutputDim < 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Pad: Cannot reduce dimension below 1, but would for tensor (name:%ls) at rank %d of size %d with prepad %d and postpad %d.", *X.GetName(), i, X.GetShape().GetData()[i], PrePad, PostPad);
					return -1;
				}
				OutputShapeData.Emplace(OutputDim);
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			OutputTensors[0]->SetShape(OutputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			using namespace UE::NNEHlslShaders::Internal;
			
			CheckInputTensorCount(InputTensorDescs.Num());
			check(OutputTensorDescs.Num() == 1);
			
			if constexpr (Version < 11)
			{
				Pads = Attributes.GetValue<TArray<int32>>(TEXT("pads"));
				Value = Attributes.GetValueOrDefault<float>(TEXT("value"), 0.0f);
			}
			FPadCS::LexFromString(Mode, *Attributes.GetValueOrDefault<FString>(TEXT("mode"), TEXT("constant")));

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;
			
			CheckInputTensorCount(InputTensors.Num());
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FPadConstants::NUM_GROUP_THREADS);

			// Set parameters
			FPadCS::FParameters* Params = GraphBuilder.AllocParameters<FPadCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			FillTensorStrideShaderParameters(Input, Params->TensorInfo, 0);
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 1);
			FillTensorSizeShaderParameters(Input, Params->TensorInfo, 2);
			for (int32 i = 0; i < Input.GetShape().Rank(); ++i)
			{
				Params->TensorInfo[i][3] = BitCast<uint32>(Pads[i]); // Pre-pad encoded as uint32
			}
			Params->Value = Value;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FPadConstants::NUM_GROUP_THREADS;

			FPadCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FPadCS::FPadMode>(Mode);
			PermutationVector.Set<FPadCS::FPadNumDimensions>(Output.GetShape().Rank());

			TShaderMapRef<FPadCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorPad, "NNE.Operator.Hlsl.Pad");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Pad.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}

	private:
		void CheckInputTensorCount(const int32 count)
		{
			if constexpr (Version < 11)
			{
				check(count == 1);
			}
			else if constexpr (Version < 18)
			{
				check(count >= 2 && count <= 3);
			}
			else
			{
				check(count >= 2 && count <= 4);
			}
		}

	};

	template<int Version>
	bool ValidatePadOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("mode"), ENNERuntimeRDGDataAttributeDataType::String);
		if constexpr (Version < 11)
		{
			AttributeValidator.AddRequired(TEXT("pads"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
			AttributeValidator.AddOptional(TEXT("value"), ENNERuntimeRDGDataAttributeDataType::Float);
		}
		bIsValid &= AttributeValidator.Validate(AttributeMap);
		
		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(3);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 1);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32, /*TemplateIdx*/ 2);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 2);
		InputValidator.AddRequired();
		if constexpr (Version >= 11)
		{
			InputValidator.AddRequired(/*TemplateIdx*/ 1);
			InputValidator.AddOptional();
		}
		if constexpr (Version >= 18)
		{
			InputValidator.AddOptional(/*TemplateIdx*/ 2);
		}
		
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<int Version>
	FOperatorHlsl* CreatePadOperator()
	{
		return new FPad<Version>();
	}

	bool RegisterPadOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		#define OP(Version) \
		Registry.OpAdd({{TEXT("Pad"), TEXT("Onnx")}, Version}, CreatePadOperator<Version>, ValidatePadOperator<Version>);

		OP(2)
		OP(11)
		OP(13)
		OP(18)

		#undef OP
		return true;
	}

} // UE::NNERuntimeRDG::Private::Hlsl
