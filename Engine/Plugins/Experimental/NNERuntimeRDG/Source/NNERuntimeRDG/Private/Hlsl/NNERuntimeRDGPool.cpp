// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGPool.h"

#include "NNEHlslShadersConvCS.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersPoolCS.h"
#include "NNEHlslShadersTypeHelper.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"


namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorMaxPool, TEXT("NNE.Operator.Hlsl.MaxPool"));
	DECLARE_GPU_STAT_NAMED(FNNEOperatorAveragePool, TEXT("NNE.Operator.Hlsl.AveragePool"));

	/**
	 * MaxPool operator implementation
	 */
	template< UE::NNEHlslShaders::Internal::EPoolOperatorType PoolOperatorType >
	class FPoolOperator : public FOperatorHlsl
	{
		static constexpr int32 SpatialInfoStrideIndex = 0;
		static constexpr int32 SpatialInfoKernelIndex = 1;
		static constexpr int32 SpatialInfoPadStartIndex = 2;
		static constexpr int32 SpatialInfoDilationIndex = 3;
	public:

		FPoolOperator() {}
		virtual ~FPoolOperator() = default;

		int32 NumSpatialDimensions = 0;
		NNEHlslShaders::Internal::EConvAutoPad AutoPad = NNEHlslShaders::Internal::EConvAutoPad::NOTSET;
		TArray<int32> Pads;
		TArray<int32> Strides;
		TArray<int32> Dilations;
		TArray<int32> KernelShape;
		int32 CeilMode = 0; // 0 is floor, 1 is ceil
		int32 KernelVolume = 0;
		EPixelFormat BufferPixelFormat;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			TConstArrayView<uint32> InputShape = X.GetShape().GetData();
			TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> OutputShape;

			check(Pads.Num() == 2*NumSpatialDimensions);
			check(Strides.Num() == NumSpatialDimensions);
			check(Dilations.Num() == NumSpatialDimensions);
			check(KernelShape.Num() == NumSpatialDimensions);

			OutputShape.SetNumUninitialized(InputShape.Num());
			OutputShape[0] = InputShape[0];
			OutputShape[1] = InputShape[1];
			for (int32 i = 0; i < NumSpatialDimensions; ++i)
			{
				//See https://github.com/onnx/onnx/blob/main/docs/Changelog.md#MaxPool-8
				switch (AutoPad)
				{
					case NNEHlslShaders::Internal::EConvAutoPad::NOTSET:
					{
						uint32 PadShape = Pads[i] + Pads[i+NumSpatialDimensions];
						float(*const RoundingFunction)(float) = 
							CeilMode == 0 ?
								(float(*)(float)) &FMath::FloorToFloat
							:
								(float(*)(float)) &FMath::CeilToFloat
							;
						OutputShape[i + 2] = (uint32)RoundingFunction((float)(InputShape[i + 2] + PadShape - ((KernelShape[i] - 1) * Dilations[i] + 1)) / (float)Strides[i] + 1.0f);
						break;
					}
					case NNEHlslShaders::Internal::EConvAutoPad::VALID:
					{
						OutputShape[i + 2] = (uint32)FMath::CeilToFloat((float)(InputShape[i + 2] - ((KernelShape[i] - 1) * Dilations[i] + 1) + 1) / (float)Strides[i]);
						break;
					}
					case NNEHlslShaders::Internal::EConvAutoPad::SAME_UPPER:
					case NNEHlslShaders::Internal::EConvAutoPad::SAME_LOWER:
					{
						OutputShape[i + 2] = (uint32)FMath::CeilToFloat((float)InputShape[i + 2] / (float)Strides[i]);
						break;
					}
				}
			}
			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);

			if constexpr (PoolOperatorType == UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL)
			{
				check(OutputTensorDescs.Num() >= 1);
				if (OutputTensorDescs.Num() > 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "MaxPool: 2nd optional output 'Indices' is not supported.");
					return false;
				}
			}
			else
			{
				check(OutputTensorDescs.Num() == 1);
			}

			const NNE::FTensorDesc& Input = InputTensorDescs[0];
			const NNE::FTensorDesc& Output = OutputTensorDescs[0];

			if (Input.GetShape().Rank() < 3)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls : Input should be at least of rank 3, to have 1+ spatial dimension(s) but is of rank %d", GetOperatorName(), Input.GetShape().Rank());
				return false;
			}
			NumSpatialDimensions = Input.GetShape().Rank() - 2;

			TArray<int32> StridesDefault;
			TArray<int32> PadsDefault;
			TArray<int32> DilationsDefault;
			StridesDefault.Init(1, NumSpatialDimensions);
			PadsDefault.Init(0, 2 * NumSpatialDimensions);
			DilationsDefault.Init(1, NumSpatialDimensions);

			NNEHlslShaders::Internal::FConvCS::LexFromString(AutoPad, *Attributes.GetValueOrDefault<FString>(TEXT("auto_pad"), TEXT("NOTSET")));
			Pads = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("pads"), PadsDefault);
			Strides = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("strides"), StridesDefault);
			Dilations = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("dilations"), DilationsDefault);
			KernelShape = Attributes.GetValue<TArray<int32>>(TEXT("kernel_shape"));

			CeilMode = Attributes.GetValueOrDefault<int32>(TEXT("ceil_mode"), CeilMode);

			if (KernelShape.Num() != NumSpatialDimensions)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls: KernelShape should have as many elements as the spatial dimensions of the input, got %d while input have %d.", GetOperatorName(), KernelShape.Num(), NumSpatialDimensions);
				return false;
			}
			if (Strides.Num() != NumSpatialDimensions)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls: Strides should have as many elements as the spatial dimensions of the input, got %d while input have %d.", GetOperatorName(), Strides.Num(), NumSpatialDimensions);
				return false;
			}
			if (Dilations.Num() != NumSpatialDimensions)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls: Dilations should have as many elements as the spatial dimensions of the input, got %d while input have %d.", GetOperatorName(), Dilations.Num(), NumSpatialDimensions);
				return false;
			}
			if (Pads.Num() != 2*NumSpatialDimensions)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls: Pads should have twice as many elements as the spatial dimensions of the input, got %d while input have %d.", GetOperatorName(), Pads.Num(), NumSpatialDimensions);
				return false;
			}

			KernelVolume = 0;//If KernelVolume is 0 the kernel will use the count of pooled elements.
			if constexpr (PoolOperatorType == UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL)
			{
				int32 CountIncludePad = Attributes.GetValueOrDefault<int32>(TEXT("count_include_pad"), 0);
				if (CountIncludePad != 0)
				{
					KernelVolume = 1;
					for (int32 dim : KernelShape)
					{
						KernelVolume *= dim;
					}
				}
			}
			BufferPixelFormat = UE::NNEHlslShaders::Internal::TensorDataTypeToPixelFormat(Input.GetDataType());

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];
			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), BufferPixelFormat));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), BufferPixelFormat));
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FPoolConstants::NUM_GROUP_THREADS);

			// Set parameters
			FPoolCS::FParameters* Params = GraphBuilder.AllocParameters<FPoolCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FPoolConstants::NUM_GROUP_THREADS;
			Params->KernelVolume = KernelVolume;
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 0);
			FillTensorStrideShaderParameters(Input, Params->TensorInfo, 1);
			FillTensorSizeShaderParameters(Input, Params->TensorInfo, 2);
			for (int32 i = 0; i < NumSpatialDimensions; ++i)
			{
				Params->SpatialInfo[i][SpatialInfoStrideIndex] = Strides[i];
				Params->SpatialInfo[i][SpatialInfoKernelIndex] = KernelShape[i];
				if (AutoPad == NNEHlslShaders::Internal::EConvAutoPad::SAME_LOWER)
				{
					// see https://github.com/onnx/onnx/blob/main/docs/Changelog.md#MaxPool-8
					// only needed for SAME_LOWER as SAME_UPPER is handled by index clamping in the HLSL kernel
					Params->SpatialInfo[i][SpatialInfoPadStartIndex] = (Output.GetShape().GetData()[i + 2] - 1) * Strides[i] + ((KernelShape[i] - 1) * Dilations[i] + 1) - Input.GetShape().GetData()[i + 2];
				}
				else
				{
					Params->SpatialInfo[i][SpatialInfoPadStartIndex] = Pads[i];
				}
				Params->SpatialInfo[i][SpatialInfoDilationIndex] = Dilations[i];
			}

			FPoolCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FPoolCS::FPoolNumSpatialDimensions>(NumSpatialDimensions);
			PermutationVector.Set<FPoolCS::FPoolType>(PoolOperatorType);

			TShaderMapRef<FPoolCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
			
			if constexpr (PoolOperatorType == EPoolOperatorType::MAX_POOL)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorMaxPool, "NNE.Operator.Hlsl.MaxPool");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.MaxPool.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					ThreadGroupCount);
			}
			else
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorAveragePool, "NNE.Operator.Hlsl.AveragePool");

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("NNE.Operator.Hlsl.AveragePool.Dispatch"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					Params,
					ThreadGroupCount);
			}
		}
	private:
		constexpr static auto GetOperatorName()
		{
			if constexpr (PoolOperatorType == UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL)
			{
				return TEXT("MaxPool");
			}
			else
			{
				return TEXT("AveragePool");
			}
		}
	};

	template< UE::NNEHlslShaders::Internal::EPoolOperatorType PoolOperatorType >
	bool ValidatePoolOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputPools)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("auto_pad"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddRequired(TEXT("kernel_shape"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("pads"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("ceil_mode"), ENNERuntimeRDGDataAttributeDataType::Int32);
		if constexpr (PoolOperatorType == UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL)
		{
			AttributeValidator.AddOptional(TEXT("dilations"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
			AttributeValidator.AddOptional(TEXT("storage_order"), ENNERuntimeRDGDataAttributeDataType::Int32);//Unused, only needed for 2nd output itself not supported, see https://github.com/onnx/onnx/issues/1370
		}
		else
		{
			AttributeValidator.AddOptional(TEXT("count_include_pad"), ENNERuntimeRDGDataAttributeDataType::Int32);
		}

		AttributeValidator.AddOptional(TEXT("strides"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateMaxPoolOperator()
	{
		return new FPoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL>();
	}

	FOperatorHlsl* CreateAveragePoolOperator()
	{
		return new FPoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL>();
	}

	bool RegisterPoolOperators(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("MaxPool"), TEXT("Onnx")}, 8}, CreateMaxPoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL>);
		Registry.OpAdd({{TEXT("MaxPool"), TEXT("Onnx")}, 10}, CreateMaxPoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL>);
		Registry.OpAdd({{TEXT("MaxPool"), TEXT("Onnx")}, 11}, CreateMaxPoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL>);
		Registry.OpAdd({{TEXT("MaxPool"), TEXT("Onnx")}, 12}, CreateMaxPoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::MAX_POOL>);
		Registry.OpAdd({ {TEXT("AveragePool"), TEXT("Onnx")}, 7}, CreateAveragePoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL>);
		Registry.OpAdd({ {TEXT("AveragePool"), TEXT("Onnx")}, 10}, CreateAveragePoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL>);
		Registry.OpAdd({ {TEXT("AveragePool"), TEXT("Onnx")}, 11}, CreateAveragePoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL>);
		Registry.OpAdd({ {TEXT("AveragePool"), TEXT("Onnx")}, 19}, CreateAveragePoolOperator, ValidatePoolOperator<UE::NNEHlslShaders::Internal::EPoolOperatorType::AVERAGE_POOL>);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
