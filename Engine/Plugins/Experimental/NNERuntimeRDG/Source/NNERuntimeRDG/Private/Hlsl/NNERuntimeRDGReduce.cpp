// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGReduce.h"

#include "Algo/Sort.h"
#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "Misc/EnumerateRange.h"
#include "Templates/Greater.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersReduceCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorReduce, TEXT("NNE.Operator.Hlsl.Reduce"));

	/**
	 * Reduce operators implementation
	 */
	template< UE::NNEHlslShaders::Internal::EReduceOperatorType ReduceOperatorType, int Version>
	class FReduceOperator : public FOperatorHlsl
	{

	public:

		FReduceOperator() = default;
		virtual ~FReduceOperator() = default;

		static constexpr bool bAxesAsInput = 
			(Version >= 13 && ReduceOperatorType == UE::NNEHlslShaders::Internal::EReduceOperatorType::Sum) ||
			(Version >= 18 && ReduceOperatorType == UE::NNEHlslShaders::Internal::EReduceOperatorType::Average)
			;

	private:

		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Axes; // Must be sorted in descending order
		int32 KeepDims = 1;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(bAxesAsInput ? (InputTensors.Num() >= 1 && InputTensors.Num() <= 2) : InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			if constexpr (bAxesAsInput)
			{
				if(InputTensors.Num() == 2)
				{
					const FTensorRef AxesTensor = InputTensors[1];
					if(!AxesTensor->HasPreparedData())
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Reduce: Tensor `axes` (name: %ls) must be CPU constant.", *InputTensors[1]->GetName());
						return false;
					}
					if(AxesTensor->GetVolume() != 0)
					{
						OperatorHelper::GetInt32ArrayFromConstTensor(Axes, AxesTensor);
					}
				}
			}

			const NNE::FTensorShape& InputShape = InputTensors[0]->GetShape();
			const int32 InputRank = InputShape.Rank();

			for (int32& Axis : Axes)
			{
				if (Axis > InputRank || Axis  < -InputRank)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Reduce: 'Axes' attribute should contain value be in the range [-r,r] with r being the rank of the input (name: %ls) however got %d while rank is %d.", *InputTensors[0]->GetName(), Axis, InputRank);
					return false;
				}

				if (Axis  < 0)
				{
					Axis += InputRank;
				}
			}

			Algo::Sort(Axes, TGreater<>());
			
			TArray<uint32> OutputShape;

			OutputShape.Reserve(InputRank);
			if(bAxesAsInput && Axes.IsEmpty())
			{
				OutputShape = InputShape.GetData();
			}
			else
			{
				for (int i = 0; i < InputRank; ++i)
				{
					if (!Axes.Contains(i))
					{
						OutputShape.Add(InputShape.GetData()[i]);
					}
					else if (KeepDims)
					{
						OutputShape.Add(1);
					}
				}
			}

			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(OutputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(bAxesAsInput ? (InputTensorDescs.Num() >= 1 && InputTensorDescs.Num() <= 2) : InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();
			
			TArray<int32> AxesDefault;
			AxesDefault.Reserve(NNE::FTensorShape::MaxRank);

			if(!bAxesAsInput || Attributes.GetValueOrDefault<int>(TEXT("noop_with_empty_axes"), 0) != 1)
			{
				for (int i = 0; i < InputRank; ++i)
				{
					AxesDefault.Add(i);
				}
			}
			
			Axes = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("axes"), AxesDefault);

			if (!bAxesAsInput && Axes.Num() == 0)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Reduce: Attribute `axes` cannot be empty.");
				return false;
			}

			KeepDims = Attributes.GetValueOrDefault<int32>(TEXT("keepdims"), 1);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(bAxesAsInput ? (InputTensors.Num() >= 1 && InputTensors.Num() <= 2) : InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != FTensorRDGRef{});
			check(OutputTensors[0] != FTensorRDGRef{});

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];
			const int32 InputRank = Input.GetShape().Rank();

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorReduce, "NNE.Operator.Hlsl.Reduce");

			if(Axes.IsEmpty())
			{
				AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Input.GetBuffer());
			}
			else
			{
				FRDGBufferRef CurrInput = Input.GetBuffer();
				TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> CurrInputShape(Input.GetShape().GetData());

				// Iterate axes in descending order
				for (TConstEnumerateRef<int32> AxisRef : EnumerateRange(Axes))
				{
					const int32 Axis = *AxisRef;

					TReduceCS::FParameters* Parameters = GraphBuilder.AllocParameters<TReduceCS::FParameters>();
					TReduceCS::FillInParameters(CurrInputShape, Axis, Parameters);

					FRDGBufferRef CurrOutput = nullptr;
					if (AxisRef.GetIndex() < Axes.Num() - 1)
					{
						const FRDGBufferDesc TempBufferDesc = FRDGBufferDesc::CreateBufferDesc(Output.GetElementByteSize(), Parameters->NumElemBeforeAxis * Parameters->NumElemAfterAxis);
						CurrOutput = GraphBuilder.CreateBuffer(TempBufferDesc, TEXT("NNE.Operator.Hlsl.Reduce.TempBuffer"), ERDGBufferFlags::None);
					}
					else // Last iteration
					{
						CurrOutput = Output.GetBuffer();
					}
					check(CurrOutput);

					TReduceCS::EnqueueRDG(GraphBuilder, Parameters, CurrInput, CurrOutput, ReduceOperatorType);
					CurrInput = CurrOutput;
					CurrInputShape[Axis] = 1;
				}
			}
		}
	};

	template< UE::NNEHlslShaders::Internal::EReduceOperatorType ReduceOperatorType, int Version>
	bool ValidateReduceOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		// This match versions 1 and 11 of the Reduces operators, next versions are 13
		// ReduceOperator are ReduceL1, ReduceL2, ReduceLogSum, ReduceLogSumExp, ReduceMax, ReduceMean, ReduceMin, ReduceProd, ReduceSum, ReduceSumSquare
		// ReduceMean-13 is also supported
		// https://github.com/onnx/onnx/blob/main/docs/Changelog.md#Reduce-1
		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("keepdims"), ENNERuntimeRDGDataAttributeDataType::Int32);
		if constexpr (!FReduceOperator<ReduceOperatorType, Version>::bAxesAsInput)
		{
			AttributeValidator.AddOptional(TEXT("axes"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		}
		else
		{
			AttributeValidator.AddOptional(TEXT("noop_with_empty_axes"), ENNERuntimeRDGDataAttributeDataType::Int32);
		}
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 1);
		InputValidator.AddRequired();
		if constexpr (FReduceOperator<ReduceOperatorType, Version>::bAxesAsInput)
		{
			InputValidator.AddOptional(/*TemplateIdx*/ 1);
		}
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template< UE::NNEHlslShaders::Internal::EReduceOperatorType ReduceOperatorType, int Version>
	FOperatorHlsl* CreateReduceOperator()
	{
		return new FReduceOperator<ReduceOperatorType, Version>();
	}

	bool RegisterReduceOperators(FOperatorRegistryHlsl& Registry)
	{
		using namespace UE::NNEHlslShaders::Internal;
		//ReduceLogSum not yet supported as multi axis case require to apply all reduction first then log.
		//ReduceSumSquare not yet supported as multi axis case require to apply square on whole tensor first then sum.
		#define OPS(Version) \
		Registry.OpAdd({ {TEXT("ReduceL1"),        TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::L1,Version>, ValidateReduceOperator<EReduceOperatorType::L1,Version>); \
		Registry.OpAdd({ {TEXT("ReduceL2"),        TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::L2,Version>, ValidateReduceOperator<EReduceOperatorType::L2,Version>); \
		Registry.OpAdd({ {TEXT("ReduceLogSumExp"), TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::LogSumExp,Version>, ValidateReduceOperator<EReduceOperatorType::LogSumExp,Version>); \
		Registry.OpAdd({ {TEXT("ReduceMax"),       TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::Max,Version>, ValidateReduceOperator<EReduceOperatorType::Max,Version>); \
		Registry.OpAdd({ {TEXT("ReduceMean"),      TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::Average,Version>, ValidateReduceOperator<EReduceOperatorType::Average,Version>); \
		Registry.OpAdd({ {TEXT("ReduceMin"),       TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::Min,Version>, ValidateReduceOperator<EReduceOperatorType::Min,Version>); \
		Registry.OpAdd({ {TEXT("ReduceProd"),      TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::Prod,Version>, ValidateReduceOperator<EReduceOperatorType::Prod,Version>); \
		Registry.OpAdd({ {TEXT("ReduceSum"),       TEXT("Onnx")}, Version}, CreateReduceOperator<EReduceOperatorType::Sum,Version>, ValidateReduceOperator<EReduceOperatorType::Sum,Version>);

		OPS(1)
		OPS(11)
		
		Registry.OpAdd({ {TEXT("ReduceSum"),       TEXT("Onnx")}, 13 }, CreateReduceOperator<EReduceOperatorType::Sum, 13>, ValidateReduceOperator<EReduceOperatorType::Sum, 13>);
		Registry.OpAdd({ {TEXT("ReduceMean"),       TEXT("Onnx")}, 13 }, CreateReduceOperator<EReduceOperatorType::Average, 18>, ValidateReduceOperator<EReduceOperatorType::Average, 13>);
		
		Registry.OpAdd({ {TEXT("ReduceMean"),       TEXT("Onnx")}, 18 }, CreateReduceOperator<EReduceOperatorType::Average, 18>, ValidateReduceOperator<EReduceOperatorType::Average, 18>);

		#undef OPS

		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
