// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGHlslOp.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	bool RegisterReduceOperators(FOperatorRegistryHlsl& Registry);
} // UE::NNERuntimeRDG::Private::Hlsl
