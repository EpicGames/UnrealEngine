// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGReshape.h"

#include "Helper/NNERuntimeRDGHelperReshape.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorReshape, TEXT("NNE.Operator.Hlsl.Reshape"));

	/**
	 * Reshape operator implementation
	 */
	class FReshape : public FOperatorHlsl
	{
	public:

		FReshape() {}
		virtual ~FReshape() = default;

		bool bAllowZero = false;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			const FTensor& ShapeTensor = *InputTensors[1];
			TArray<uint32> OutputShapeData;

			check(ShapeTensor.GetDataType() == ENNETensorDataType::Int64);

			if (!ShapeTensor.HasPreparedData())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Reshape: Input 'Shape' (name: %ls) should be constant for shape inference to succeed, however it is not.", *ShapeTensor.GetName());
				return -1;
			}

			if (!ShapeHelper::Reshape::ReshapeTensor<int64>(X.GetShape(), ShapeTensor, bAllowZero, OutputShapeData))
			{
				return -1;
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			
			OutputTensors[0]->SetShape(OutputShape);
			if (X.HasPreparedData())
			{
				OutputTensors[0]->SetPreparedData<uint8>(X.GetPreparedData<uint8>());
			}
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);

			bAllowZero = (bool) (Attributes.GetValueOrDefault<int32>(TEXT("allowzero"), 0));
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorReshape, "NNE.Operator.Hlsl.Reshape");

			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Data.GetBuffer());
		}
	};

	bool ValidateReshapeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputReshapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("allowzero"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();

		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 1);
		InputValidator.AddRequired(1);

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateReshapeOperator()
	{
		return new FReshape();
	}

	bool RegisterReshapeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Reshape"), TEXT("Onnx")}, 5}, CreateReshapeOperator, ValidateReshapeOperator);
		Registry.OpAdd({{TEXT("Reshape"), TEXT("Onnx")}, 13}, CreateReshapeOperator, ValidateReshapeOperator);
		Registry.OpAdd({{TEXT("Reshape"), TEXT("Onnx")}, 14}, CreateReshapeOperator, ValidateReshapeOperator);
		Registry.OpAdd({{TEXT("Reshape"), TEXT("Onnx")}, 19}, CreateReshapeOperator, ValidateReshapeOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
