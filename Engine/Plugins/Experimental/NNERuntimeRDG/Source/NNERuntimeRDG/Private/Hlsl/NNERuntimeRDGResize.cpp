// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGResize.h"

#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersResizeCS.h"
#include "NNEHlslShadersTypeHelper.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorResize, TEXT("NNE.Operator.Hlsl.Resize"));


	/**
	 * Resize operator implementation
	 */
	class FResize : public FOperatorHlsl
	{
	public:

		FResize() {}
		virtual ~FResize() = default;

	private:

		enum EKeepAspectRatioPolicy
		{
			Stretch,
			NotLarger,
			NotSmaller
		};

		static EKeepAspectRatioPolicy KeepAspectRatioPolicyFromString(const TCHAR* StringVal)
		{
			EKeepAspectRatioPolicy OutValue = EKeepAspectRatioPolicy::Stretch;
			if (FCString::Stricmp(StringVal, TEXT("stretch")) == 0) OutValue = EKeepAspectRatioPolicy::Stretch;
			else if (FCString::Stricmp(StringVal, TEXT("not_larger")) == 0) OutValue = EKeepAspectRatioPolicy::NotLarger;
			else if (FCString::Stricmp(StringVal, TEXT("not_smaller")) == 0) OutValue = EKeepAspectRatioPolicy::NotSmaller;

			return OutValue;
		}
		
		TArray<int32> Axes;
		UE::NNEHlslShaders::Internal::ECoordTransMode CoordTransMode;
		float CubicCoeffA;
		int32 ExcludeOutside;
		float ExtrapolationValue;
		EKeepAspectRatioPolicy KeepAspectRatioPolicy;
		UE::NNEHlslShaders::Internal::EMode Mode;
		UE::NNEHlslShaders::Internal::ENearestMode NearestMode;
		TArray<float> ScalesData;
		TArray<float> Adjustments; // Necessary for "half_pixel_symmetric" coordinate transform mode
		TArray<float> RegionOfInterest;
		EPixelFormat BufferPixelFormat;
		

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() >= 3 && InputTensors.Num() <= 4);
			check(OutputTensors.Num() == 1);

			const FTensor& Input = *InputTensors[0];
			const FTensor& Roi = *InputTensors[1];

			if(CoordTransMode == UE::NNEHlslShaders::Internal::ECoordTransMode::TfCropAndResize)
			{
				if(!Roi.HasPreparedData())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `roi` tensor could not be made constant. (name %ls).", *Roi.GetName());
					return -1;
				}
				
				RegionOfInterest = Roi.GetPreparedData<float>();

				if(RegionOfInterest.Num() != 2 * Input.GetShape().Rank())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `roi` tensor (name %ls) must have 2 * N length.", *Roi.GetName());
					return -1;
				}
			}

			TArray<uint32> OutputShapeData;
			OutputShapeData.SetNumUninitialized(Input.GetShape().Rank());

			ScalesData.SetNumUninitialized(Input.GetShape().Rank());
		
			if(InputTensors.Num() == 3)
			{
				const FTensor& Scales = *InputTensors[2];
				if(!Scales.HasPreparedData())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `scales` tensor could not be made constant. (name %ls).", *Scales.GetName());
					return -1;
				}

				if(CoordTransMode == UE::NNEHlslShaders::Internal::ECoordTransMode::HalfPixelSymmetric)
				{
					Adjustments.SetNumUninitialized(Input.GetShape().Rank());
				}

				for (int32 Idx = 0; Idx < Input.GetShape().Rank(); ++Idx)
				{
					OutputShapeData[Idx] = (uint32) FMath::FloorToInt32(
						//NOTE: documentation erroneously says that for coordinate_transform_mode = tf_crop_and_resize the output shape should be computed as:
						// (float) Input.GetShape().GetData()[Idx] * (Roi.GetShape().GetData()[Idx + Input.GetShape().Rank()] - Roi.GetShape().GetData()[Idx]) * Scales.GetPreparedData<float>()[Idx]
						// however it should be computed like for any other coordinate_transform_mode:
							(float) Input.GetShape().GetData()[Idx] * Scales.GetPreparedData<float>()[Idx]
						);

					ScalesData[Idx] = Scales.GetPreparedData<float>()[Idx];
					
					if(CoordTransMode == UE::NNEHlslShaders::Internal::ECoordTransMode::HalfPixelSymmetric)
					{
						Adjustments[Idx] = (float) OutputShapeData[Idx] / ((float) Input.GetShape().GetData()[Idx] * Scales.GetPreparedData<float>()[Idx]);
					}
				}
			}
			
			if(InputTensors.Num() == 4)
			{
				const FTensor& Sizes = *InputTensors[3];
				if(!Sizes.HasPreparedData())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `sizes` tensor could not be made constant. (name %ls).", *Sizes.GetName());
					return -1;
				}

				TArray<float> Ratios;
				{
					Ratios.SetNumUninitialized(Input.GetShape().Rank());
					for (int32 Idx = 0; Idx < Input.GetShape().Rank(); ++Idx)
					{
						Ratios[Idx] = (float) Sizes.GetPreparedData<int64>()[Idx] / (float) Input.GetShape().GetData()[Idx];
					}
				}
				float MinRatio = *Algo::MinElement(Ratios);
				float MaxRatio = *Algo::MaxElement(Ratios);

				for (int32 Idx = 0; Idx < Input.GetShape().Rank(); ++Idx)
				{
					switch(KeepAspectRatioPolicy)
					{
					case EKeepAspectRatioPolicy::Stretch:
						OutputShapeData[Idx] = Sizes.GetPreparedData<int64>()[Idx];
						break;
					case EKeepAspectRatioPolicy::NotLarger:
						OutputShapeData[Idx] = (uint32) FMath::RoundToInt32(MinRatio * Input.GetShape().GetData()[Idx]);
						break;
					case EKeepAspectRatioPolicy::NotSmaller:
						OutputShapeData[Idx] = (uint32) FMath::RoundToInt32(MaxRatio * Input.GetShape().GetData()[Idx]);
						break;
					};

					ScalesData[Idx] = (float) OutputShapeData[Idx] / (float) Input.GetShape().GetData()[Idx];
				}
				
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			OutputTensors[0]->SetShape(OutputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() >= 1 && InputTensorDescs.Num() <= 4);
			check(OutputTensorDescs.Num() == 1);

			// NOTE: implementation for antialiasing is missing
			if(Attributes.GetValueOrDefault<int32>(TEXT("antialias"), 0) == 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: antialias not yet supported.");
				return false;
			}

			Axes = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("axes"), TArray<int32>{});
			
			for (int32& Axis : Axes)
			{
				if (Axis > InputTensorDescs[0].GetShape().Rank() || Axis < -InputTensorDescs[0].GetShape().Rank())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: 'Axes' attribute should contain value be in the range [-r,r] with r being the rank of the input (name: %ls) however got %d while rank is %d.", *InputTensorDescs[0].GetName(), Axis, InputTensorDescs[0].GetShape().Rank());
					return false;
				}

				if (Axis < 0)
				{
					Axis += InputTensorDescs[0].GetShape().Rank();
				}
			}

			// NOTE: `axes` attribute not yet supported
			if(Axes.Num() != 0)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `axes` attribute not yet supported.");
				return false;
			}

			CoordTransMode = UE::NNEHlslShaders::Internal::FResizeCS::CoordTransModeFromString(*Attributes.GetValueOrDefault<FString>(TEXT("coordinate_transformation_mode"), TEXT("half_pixel")));
			CubicCoeffA = Attributes.GetValueOrDefault<float>(TEXT("cubic_coeff_a"), -0.75f);
			ExcludeOutside = Attributes.GetValueOrDefault<int32>(TEXT("exclude_outside"), 0);
			// NOTE: `exclude_outside` attribute not yet supported
			if(ExcludeOutside == 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: `exclude_outside` attribute not yet supported.");
				return false;
			}

			ExtrapolationValue = Attributes.GetValueOrDefault<float>(TEXT("extrapolation_value"), 0.0f);
			KeepAspectRatioPolicy = KeepAspectRatioPolicyFromString(*Attributes.GetValueOrDefault<FString>(TEXT("keep_aspect_ratio_policy"), TEXT("stretch")));
			Mode = UE::NNEHlslShaders::Internal::FResizeCS::ModeFromString(*Attributes.GetValueOrDefault<FString>(TEXT("mode"), TEXT("nearest")));
			//NOTE: cubic interpolation not yet supported
			if(Mode == UE::NNEHlslShaders::Internal::EMode::Cubic)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Cubic interpolation not yet supported.");
				return false;
			}

			NearestMode = UE::NNEHlslShaders::Internal::FResizeCS::NearestModeFromString(*Attributes.GetValueOrDefault<FString>(TEXT("nearest_mode"), TEXT("round_prefer_floor")));
			BufferPixelFormat = UE::NNEHlslShaders::Internal::TensorDataTypeToPixelFormat(InputTensorDescs[0].GetDataType());
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 1 && InputTensors.Num() <= 4);
			check(OutputTensors.Num() == 1);
			for(int Idx = 0; Idx < InputTensors.Num(); ++Idx)
			{
				check(InputTensors[Idx] != nullptr);
			}
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorResize, "NNE.Operator.Hlsl.Resize");

			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), BufferPixelFormat));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), BufferPixelFormat));

			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FResizeConstants::NUM_GROUP_THREADS);

			// Set parameters
			FResizeCS::FParameters* Params = GraphBuilder.AllocParameters<FResizeCS::FParameters>();
			
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FResizeConstants::NUM_GROUP_THREADS;

			FillTensorStrideShaderParameters(Input, Params->InputTensorInfo, /*Idx*/0);
			FillTensorSizeShaderParameters(Input, Params->InputTensorInfo, /*Idx*/1);
			FillTensorStrideShaderParameters(Output, Params->OutputTensorInfo, /*Idx*/0);
			FillTensorSizeShaderParameters(Output, Params->OutputTensorInfo, /*Idx*/1);

			for(int DimIdx = 0; DimIdx < Input.GetShape().Rank(); ++DimIdx)
			{
				// NOTE: floats are encoded as unit32s and then decoded in the shader code

				if(CoordTransMode == UE::NNEHlslShaders::Internal::ECoordTransMode::HalfPixelSymmetric)
				{
					// Set adjustments
					Params->OutputTensorInfo[DimIdx][/*Idx*/ 2] = BitCast<uint32>(Adjustments[DimIdx]);
				}
				Params->ScalesData[DimIdx][/*Idx*/ 0] = ScalesData[DimIdx];

				if(CoordTransMode == UE::NNEHlslShaders::Internal::ECoordTransMode::TfCropAndResize)
				{
					// Set ROI start indices
					Params->InputTensorInfo[DimIdx][/*Idx*/ 2] = BitCast<uint32>(RegionOfInterest[DimIdx]);
					// Set ROI end indices
					Params->InputTensorInfo[DimIdx][/*Idx*/ 3] = BitCast<uint32>(RegionOfInterest[Input.GetShape().Rank() + DimIdx]);
				}
			}

			FResizeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FResizeCS::FResizeNumDimensions>(Output.GetShape().Rank());
			PermutationVector.Set<FResizeCS::FMode>(Mode);
			PermutationVector.Set<FResizeCS::FNearestMode>(NearestMode);
			PermutationVector.Set<FResizeCS::FCoordTransMode>(CoordTransMode);

			TShaderMapRef<FResizeCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorResize, "NNE.Operator.Hlsl.Resize");
			
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Resize.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateResizeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		using namespace UE::NNEHlslShaders::Internal;
		
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("antialias"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("axes"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddOptional(TEXT("coordinate_transformation_mode"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddOptional(TEXT("cubic_coeff_a"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("exclude_outside"), ENNERuntimeRDGDataAttributeDataType::Int32);
		AttributeValidator.AddOptional(TEXT("extrapolation_value"), ENNERuntimeRDGDataAttributeDataType::Float);
		AttributeValidator.AddOptional(TEXT("keep_aspect_ratio_policy"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddOptional(TEXT("mode"), ENNERuntimeRDGDataAttributeDataType::String);
		AttributeValidator.AddOptional(TEXT("nearest_mode"), ENNERuntimeRDGDataAttributeDataType::String);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(3);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float, 1/*TemplateIdx*/);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 2/*TemplateIdx*/);

		InputValidator.AddRequired();
		InputValidator.AddOptional(1/*TemplateIdx*/);
		InputValidator.AddOptional(1/*TemplateIdx*/);
		InputValidator.AddOptional(2/*TemplateIdx*/);

		if (!InputValidator.Validate(InputTypes))
		{
			return false;
		}

		if(InputTypes.Num() < 3 || InputTypes.Num() > 4)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Got a total of '%d' inputs but should be between 3 and 4.", InputTypes.Num());
			return false;
		}

		if(InputShapes[0].Rank() < 1)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Input tensor must have rank >= 1.");
			return false;
		}

		if (InputTypes[1] != ENNETensorDataType::None)
		{
			if(InputShapes[1].Rank() != 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Roi tensor must be a 1-D tensor.");
				return false;
			}
			FString TransMode = AttributeMap.GetValueOrDefault<FString>(TEXT("coordinate_transformation_mode"), TEXT("half_pixel"));
			bool IsCropAndResizeMode = FResizeCS::CoordTransModeFromString(*TransMode) == ECoordTransMode::TfCropAndResize;
			if(IsCropAndResizeMode && InputShapes[1].GetData()[0] != 2 * InputShapes[0].Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Roi tensor must have dimension 2*N (where N is the input rank) when `coordinate_transformation_mode` is `tf_crop_and_resize`.");
				return false;
			}
		}

		if(InputTypes.Num() == 3)
		{
			if(InputShapes[2].Rank() != 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Scales tensor must be a 1-D tensor.");
				return false;
			}
			if(InputShapes[2].GetData()[0] != InputShapes[0].Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Scales tensor must have dimension N (where N is the input rank).");
				return false;
			}
		}
		else
		{
			if(InputTypes[2] != ENNETensorDataType::None)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Scales tensor must be empty (i.e. empty name and data type 'None') when Sizes is specified.");
				return false;
			}
			if(InputShapes[3].Rank() != 1)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Sizes tensor must be a 1-D tensor.");
				return false;
			}
			if(InputShapes[3].GetData()[0] != InputShapes[0].Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Resize: Sizes tensor must have dimension N (where N is the input rank).");
				return false;
			}
		}

		return bIsValid;
	}

	FOperatorHlsl* CreateResizeOperator()
	{
		return new FResize();
	}

	bool RegisterResizeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		
		// Note: CumSum is currently not working on Mac.
		#if !PLATFORM_MAC
			Registry.OpAdd({{TEXT("Resize"), TEXT("Onnx")}, 10}, CreateResizeOperator, ValidateResizeOperator);
			Registry.OpAdd({{TEXT("Resize"), TEXT("Onnx")}, 11}, CreateResizeOperator, ValidateResizeOperator);
			Registry.OpAdd({{TEXT("Resize"), TEXT("Onnx")}, 13}, CreateResizeOperator, ValidateResizeOperator);
			Registry.OpAdd({{TEXT("Resize"), TEXT("Onnx")}, 18}, CreateResizeOperator, ValidateResizeOperator);
			Registry.OpAdd({{TEXT("Resize"), TEXT("Onnx")}, 19}, CreateResizeOperator, ValidateResizeOperator);
		#endif
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
