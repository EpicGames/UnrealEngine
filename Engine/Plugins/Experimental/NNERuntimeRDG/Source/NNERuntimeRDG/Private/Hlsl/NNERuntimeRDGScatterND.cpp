// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGScatterND.h"

#include "Algo/Compare.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersScatterNDCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorScatterND, TEXT("NNE.Operator.Hlsl.ScatterND"));

	/**
	 * ScatterND operator implementation
	 */
	class FScatterND : public FOperatorHlsl
	{
	public:

		FScatterND() {}
		virtual ~FScatterND() = default;

	private:
		UE::NNEHlslShaders::Internal::EScatterNDReductionType ReductionType;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 3);
			check(OutputTensors.Num() == 1);

			const FTensor& Input = *InputTensors[0];
			TConstArrayView<uint32> InputShape = Input.GetShape().GetData();
			const FTensor& Indices = *InputTensors[1];
			TConstArrayView<uint32> IndicesShape = Indices.GetShape().GetData();
			const FTensor& Updates = *InputTensors[2];
			TConstArrayView<uint32> UpdatesShape = Updates.GetShape().GetData();

			if (IndicesShape.Last() > (uint32) InputShape.Num())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ScatterND: Last dimension in the shape of `indices` (%d) must be less than input rank (%d).", IndicesShape.Last(), InputShape.Num());
				return -1;
			}
			if (UpdatesShape.Num() != (IndicesShape.Num() - 1) + (InputShape.Num() - IndicesShape.Last()))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ScatterND: Rank of `updates` should equal (q - 1) + (r - k), with q rank of `indices`, r rank of `data` and `k` last dimension of `indices`' shape.");
				return -1;
			}
			if (!Algo::Compare( UpdatesShape.Slice(0, IndicesShape.Num() - 1), IndicesShape.Slice(0, IndicesShape.Num() - 1) ))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ScatterND: Updates.shape[0:q-1] should match indices.shape[0:q-1].");
				return -1;
			}
			if (!Algo::Compare( UpdatesShape.RightChop(IndicesShape.Num() - 1), InputShape.RightChop(IndicesShape.Last()) ))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ScatterND: Updates.shape[q-1:] should match data.shape[k:].");
				return -1;
			}
			//NOTE: we need to limit the volume of Input to the int32 maximum expressible value due to unavailability of int64 type in shaders.
			if (Input.GetShape().Volume() > (uint64) MAX_int32)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ScatterND: Only supports input tensors up to a volume of %u.", MAX_int32);
				return -1;
			}

			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(InputShape));

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
            using namespace UE::NNEHlslShaders::Internal;
			check(InputTensorDescs.Num() == 3);
			check(OutputTensorDescs.Num() == 1);

			const NNE::FTensorDesc& InputDesc = InputTensorDescs[0];
			const int32 InputRank = InputDesc.GetShape().Rank();
			
			ReductionType = FScatterNDCS::ReductionFromString(*Attributes.GetValueOrDefault<FString>(TEXT("reduction"), TEXT("none")));

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 3);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(InputTensors[2] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Indices = *InputTensors[1];
			const FTensorRDG& Updates = *InputTensors[2];
			const FTensorRDG& Output = *OutputTensors[0];

			// NOTE: Indices tensor is int64, but UE lacks support of int64 buffers. Here we use a 32-bit pixel format and in the shader we reinterpret two words as int64.
			const FRDGBufferSRVRef InputIndicesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Indices.GetBuffer(), PF_R32_SINT));
			const FRDGBufferSRVRef InputUpdatesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Updates.GetBuffer(), PF_R32_FLOAT));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Updates.GetVolume(), FScatterNDConstants::NUM_GROUP_THREADS);

			// Set parameters
			FScatterNDCS::FParameters* Params = GraphBuilder.AllocParameters<FScatterNDCS::FParameters>();
			
			Params->InputIndices = InputIndicesSRV;
			Params->InputUpdates = InputUpdatesSRV;
			Params->Output = OutputUAV;
			Params->Num = Updates.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FScatterNDConstants::NUM_GROUP_THREADS;

			FillTensorStrideShaderParameters(Input, Params->DataTensorInfo, /*Idx*/0);
			FillTensorSizeShaderParameters(Input, Params->DataTensorInfo, /*Idx*/1);
			static_assert(NNE::FTensorShape::MaxRank <= NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS);
			uint32 PartialIndexRank = Indices.GetShape().GetData().Last();
			check(PartialIndexRank <= (uint32) Input.GetShape().Rank());
			Params->PartialIndexRank = PartialIndexRank;


			auto ComputeSliceSize = [] (TConstArrayView<uint32> DataShape, uint32 SliceStartIdx) -> uint32 
				{
					uint32 Res = 1;
					for(uint32 DimIdx = SliceStartIdx; DimIdx < (uint32) DataShape.Num(); ++DimIdx)
					{
						Res *= DataShape[DimIdx];
					}
					return Res;
				};
			Params->SliceSize = ComputeSliceSize(Input.GetShape().GetData(), PartialIndexRank);

			FScatterNDCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FScatterNDCS::FScatterNDNumDimensions>(Output.GetShape().Rank());
			PermutationVector.Set<FScatterNDCS::FReduceType>(ReductionType);

			TShaderMapRef<FScatterNDCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorScatterND, "NNE.Operator.Hlsl.ScatterND");
			
			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Input.GetBuffer());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.ScatterND.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateScatterNDOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputScatterNDs)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("reduction"), ENNERuntimeRDGDataAttributeDataType::String);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /*TemplateIdx*/ 1);
		InputValidator.AddRequired();
		InputValidator.AddRequired(/*TemplateIdx*/ 1);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateScatterNDOperator()
	{
		return new FScatterND();
	}

	bool RegisterScatterNDOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("ScatterND"), TEXT("Onnx")}, 11}, CreateScatterNDOperator, ValidateScatterNDOperator);
		Registry.OpAdd({{TEXT("ScatterND"), TEXT("Onnx")}, 13}, CreateScatterNDOperator, ValidateScatterNDOperator);
		Registry.OpAdd({{TEXT("ScatterND"), TEXT("Onnx")}, 16}, CreateScatterNDOperator, ValidateScatterNDOperator);
		Registry.OpAdd({{TEXT("ScatterND"), TEXT("Onnx")}, 18}, CreateScatterNDOperator, ValidateScatterNDOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
