// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGShape.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	/**
	 * Shape operator implementation
	 */
	class FShape : public FOperatorHlsl
	{
	public:

		FShape() {}
		virtual ~FShape() = default;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(OutputTensors[0]->GetDataType() == ENNETensorDataType::Int64);

			const FTensor& X = *InputTensors[0];

			TArray<uint32> OutputShapeData;
			
			OutputShapeData.Emplace(X.GetShape().Rank());
			
			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			TArray<int64> OutputData;
			
			OutputTensors[0]->SetShape(OutputShape);
			for (int32 i = 0; i < X.GetShape().Rank(); ++i)
			{
				OutputData.Emplace(X.GetShape().GetData()[i]);
			}
			OutputTensors[0]->SetPreparedData<int64>(OutputData);
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			if (OutputTensorDescs[0].GetDataType() != ENNETensorDataType::Int64)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Shape: Should output a tensor of type Int64");
				return false;
			}
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Shape: Output should be constant and already uploaded to GPU memory. Dispatch should not need to be called.");
		}
	};

	bool ValidateShapeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateShapeOperator()
	{
		return new FShape();
	}

	bool RegisterShapeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Shape"), TEXT("Onnx")}, 1}, CreateShapeOperator, ValidateShapeOperator);
		Registry.OpAdd({{TEXT("Shape"), TEXT("Onnx")}, 13}, CreateShapeOperator, ValidateShapeOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
