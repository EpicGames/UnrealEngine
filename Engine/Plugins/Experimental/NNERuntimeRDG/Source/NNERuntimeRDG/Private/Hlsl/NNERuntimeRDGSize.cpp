// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGSize.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	/**
	 * Size operator implementation
	 */
	class FSize : public FOperatorHlsl
	{
	public:

		FSize() {}
		virtual ~FSize() = default;

		TArray<int32> Axes;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			NNE::FTensorShape ScalarShape = NNE::FTensorShape::Make({});
			TArray<int64> OutputData;
			
			OutputTensors[0]->SetShape(ScalarShape);
			OutputData.Add(X.GetShape().Volume());
			OutputTensors[0]->SetPreparedData<int64>(OutputData);
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);
			check(OutputTensorDescs[0].GetDataType() == ENNETensorDataType::Int64);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Size: Output should be constant and already uploaded to GPU memory. Dispatch should not need to be called.");
		}
	};

	bool ValidateSizeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputSizes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateSizeOperator()
	{
		return new FSize();
	}

	bool RegisterSizeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Size"), TEXT("Onnx")}, 1}, CreateSizeOperator, ValidateSizeOperator);
		Registry.OpAdd({{TEXT("Size"), TEXT("Onnx")}, 13}, CreateSizeOperator, ValidateSizeOperator);
		Registry.OpAdd({{TEXT("Size"), TEXT("Onnx")}, 19}, CreateSizeOperator, ValidateSizeOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
