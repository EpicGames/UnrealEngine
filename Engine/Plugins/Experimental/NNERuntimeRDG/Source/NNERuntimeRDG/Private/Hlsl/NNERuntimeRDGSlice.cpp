// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGSlice.h"

#include "Helper/NNERuntimeRDGHelperSlice.h"
#include "Helper/NNERuntimeRDGOperatorHelper.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersSliceCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorSlice, TEXT("NNE.Operator.Hlsl.Slice"));

	/**
	 * Slice operator implementation
	 */
	template<int32 OpVersion>
	class FSlice : public FOperatorHlsl
	{
	public:

		FSlice() {}
		virtual ~FSlice() = default;

	private:

		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> AxesAttr;
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> EndsAttr;
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> StartsAttr;
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> StepsAttr;

		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Start;
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> End;
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Step;

		bool TryGetAttributesFromConstantTensors(TConstArrayView<FTensorRef> InputTensors)
		{
			check(InputTensors.Num() >= 3 && InputTensors.Num() <= 5);

			if (InputTensors.Num() >= 4)
			{
				if (!OperatorHelper::GetInt32ArrayFromConstTensor(AxesAttr, InputTensors[3]))
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Axes' input tensor (%ls) is only supported as a constant integer tensor but it is not.", *InputTensors[3]->GetName());
					return false;
				}
				if(AxesAttr.Num() > InputTensors[0]->GetShape().Rank() || AxesAttr.Num() < 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Axes' input tensor (%ls) contains %d elements but input rank is %d.", *InputTensors[3]->GetName(), AxesAttr.Num(), InputTensors[0]->GetShape().Rank());
					return false;
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InputTensors[0]->GetShape().Rank(); ++Idx)
				{
					AxesAttr.Add(Idx);
				}
			}

			const int32 NumAxes = AxesAttr.Num();

			if (!OperatorHelper::GetInt32ArrayFromConstTensor(StartsAttr, InputTensors[1]))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Starts' input tensor (%ls) is only supported as a constant integer tensor but it is not.", *InputTensors[1]->GetName());
				return false;
			}
			if(StartsAttr.Num() != NumAxes)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Starts' input tensor (%ls) contains %d elements but number of axes is %d.", *InputTensors[1]->GetName(), StartsAttr.Num(), NumAxes);
				return false;
			}

			if (!OperatorHelper::GetInt32ArrayFromConstTensor(EndsAttr, InputTensors[2]))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Ends' input tensor (%ls) is only supported as a constant integer tensor but it is not.", *InputTensors[2]->GetName());
				return false;
			}
			if(EndsAttr.Num() != NumAxes)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Ends' input tensor (%ls) contains %d elements but number of axes is %d.", *InputTensors[2]->GetName(), EndsAttr.Num(), NumAxes);
				return false;
			}

			

			if (InputTensors.Num() == 5)
			{
				if (!OperatorHelper::GetInt32ArrayFromConstTensor(StepsAttr, InputTensors[4]))
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Steps' input tensor (%ls) is only supported as a constant integer tensor but it is not.", *InputTensors[4]->GetName());
					return false;
				}
				if(StepsAttr.Num() != NumAxes)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Steps' input tensor (%ls) contains %d elements but number of axes is %d.", *InputTensors[4]->GetName(), StepsAttr.Num(), NumAxes);
					return false;
				}
				for(const int32 Value : StepsAttr)
				{
					if(Value == 0)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: 'Steps' tensor (%ls) can only contain non-0 integers.", *InputTensors[4]->GetName());
						return false;
					}
				}
			}
			else
			{
				StepsAttr.Init(1, AxesAttr.Num()); // Default for steps is all 1s
			}

			return true;
		}

		void ComputeStartAndEndFromInputShape(TConstArrayView<uint32> InputShapeData)
		{
			const int32 InputRank = InputShapeData.Num();

			check(AxesAttr.Num() <= InputRank);
			check(AxesAttr.Num() == StartsAttr.Num());
			check(AxesAttr.Num() == EndsAttr.Num());
			check(AxesAttr.Num() == StepsAttr.Num());

			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Axes(AxesAttr);
			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Starts(StartsAttr);
			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Ends(EndsAttr);
			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>>& Steps(StepsAttr);

			//see https://github.com/onnx/onnx/blob/main/docs/Operators.md#slice for algorithm
			Start.SetNum(InputRank);
			End.SetNum(InputRank);
			Step.SetNum(InputRank);
			for (int32 i = 0; i < InputRank; ++i)
			{
				Start[i] = 0;
				End[i] = InputShapeData[i];
				Step[i] = 1;
			}
			for (int32 i = 0; i < Axes.Num(); ++i)
			{
				if (Axes[i] < 0)
				{
					Axes[i] += InputRank;
				}
			}
			for (int32 i = 0; i < Starts.Num(); ++i)
			{
				if (Starts[i] < 0)
				{
					Starts[i] += InputShapeData[Axes[i]];
				}
			}
			for (int32 i = 0; i < Ends.Num(); ++i)
			{
				if (Ends[i] < 0)
				{
					Ends[i] += InputShapeData[Axes[i]];
				}
			}
			for (int32 i = 0; i < Axes.Num(); ++i)
			{
				if(Steps[i] > 0)
				{
					Start[Axes[i]] = FMath::Clamp(Starts[i], 0, InputShapeData[Axes[i]]);
					End[Axes[i]] = FMath::Clamp(Ends[i], 0, InputShapeData[Axes[i]]);
				}
				else
				{
					Start[Axes[i]] = FMath::Clamp(Starts[i], 0, InputShapeData[Axes[i]] - 1);
					End[Axes[i]] = FMath::Clamp(Ends[i], -1, InputShapeData[Axes[i]] - 1);
				}
				Step[Axes[i]] = Steps[i];
			}
		}

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(OutputTensors.Num() == 1);

			if constexpr (OpVersion == 1)
			{
				check(InputTensors.Num() == 1);
			}
			else
			{
				check(InputTensors.Num() >= 3 && InputTensors.Num() <= 5);
				if (!TryGetAttributesFromConstantTensors(InputTensors))
				{
					return -1;
				}
			}

			TConstArrayView<uint32> InputShapeData(InputTensors[0]->GetShape().GetData());
			const int32 InputRank = InputShapeData.Num();

			ComputeStartAndEndFromInputShape(InputShapeData);

			TArray<uint32> OutputShapeData;

			OutputShapeData.Reserve(InputRank);
			for (int32 Idx = 0; Idx < InputRank; ++Idx)
			{
				int32 RangeSize = Step[Idx] > 0 ? End[Idx] - Start[Idx] : Start[Idx] - End[Idx];
				if(RangeSize < 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: Start (tensor: %ls) and end (tensor: %ls) indices are incompatible with step direction for dimension %d.", *InputTensors[1]->GetName(), *InputTensors[2]->GetName(), Idx);
					return -1;
				}
				uint32 OutDimSize = FMath::DivideAndRoundUp(RangeSize, FMath::Abs(Step[Idx]));
				OutputShapeData.Add(OutDimSize);
			}
			
			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);

			OutputTensors[0]->SetShape(OutputShape);
			
			CPUHelper::Slice::Apply(*InputTensors[0], *OutputTensors[0], Start, Step);

			if (InputTensors[0]->HasPreparedData() && !OutputTensors[0]->HasPreparedData())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: Output tensor (%ls) could not be constant-folded from input.", *OutputTensors[0]->GetName());
				return -1;
			}

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(OutputTensorDescs.Num() == 1);

			if constexpr (OpVersion == 1)
			{
				check(InputTensorDescs.Num() == 1);

				EndsAttr = Attributes.GetValue<TArray<int32>>(TEXT("ends"));
				StartsAttr = Attributes.GetValue<TArray<int32>>(TEXT("starts"));

				TArray<int32> AxesDefault;

				for (int32 i = 0; i < StartsAttr.Num(); ++i)
				{
					AxesDefault.Add(i);
				}

				AxesAttr = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("axes"), AxesDefault);

				if (EndsAttr.Num() != StartsAttr.Num() || AxesAttr.Num() != StartsAttr.Num())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Slice: Starts, Ends and Axes must be of the same size.");
					return false;
				}

				StepsAttr.Init(1, AxesAttr.Num()); // Default for steps is all 1s
			}
			else
			{
				check(InputTensorDescs.Num() >= 3 && InputTensorDescs.Num() <= 5);
			}

			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];
			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FSliceConstants::NUM_GROUP_THREADS);

			// Set parameters
			FSliceCS::FParameters* Params = GraphBuilder.AllocParameters<FSliceCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FSliceConstants::NUM_GROUP_THREADS;
			FillTensorStrideShaderParameters(Input, Params->TensorInfo, 0);
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 1);
			static_assert(NNE::FTensorShape::MaxRank <= NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS);
			check(Start.Num() == Input.GetShape().Rank());
			for (int32 DimIdx = 0; DimIdx < Input.GetShape().Rank(); ++DimIdx)
			{
				Params->TensorInfo[DimIdx][/*SlotIdx*/ 2] = static_cast<uint32>(Start[DimIdx]);
				Params->TensorInfo[DimIdx][/*SlotIdx*/ 3] = BitCast<uint32>(Step[DimIdx]);
			}

			FSliceCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSliceCS::FSliceNumDimensions>(Output.GetShape().Rank());

			TShaderMapRef<FSliceCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorSlice, "NNE.Operator.Hlsl.Slice");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Slice.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateSliceOperatorOpset1(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputSlices)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axes"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddRequired(TEXT("ends"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		AttributeValidator.AddRequired(TEXT("starts"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	bool ValidateSliceOperatorOpset10To13(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputSlices)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);

		InputValidator.AddSupportedType(ENNETensorDataType::Float, 0);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32, 0);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 0);
		InputValidator.AddRequired(0);//Data

		InputValidator.AddSupportedType(ENNETensorDataType::Int32, 1);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, 1);
		InputValidator.AddRequired(1);//Starts
		InputValidator.AddRequired(1);//Ends
		InputValidator.AddOptional(1);//Axes
		InputValidator.AddOptional(1);//Steps

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<int32 OpsetVersion>
	FOperatorHlsl* CreateSliceOperator()
	{
		return new FSlice<OpsetVersion>();
	}

	bool RegisterSliceOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({ {TEXT("Slice"), TEXT("Onnx")}, 1  }, CreateSliceOperator<1>,  ValidateSliceOperatorOpset1);
		Registry.OpAdd({ {TEXT("Slice"), TEXT("Onnx")}, 10 }, CreateSliceOperator<10>, ValidateSliceOperatorOpset10To13);
		Registry.OpAdd({ {TEXT("Slice"), TEXT("Onnx")}, 11 }, CreateSliceOperator<11>, ValidateSliceOperatorOpset10To13);
		Registry.OpAdd({ {TEXT("Slice"), TEXT("Onnx")}, 13 }, CreateSliceOperator<13>, ValidateSliceOperatorOpset10To13);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
