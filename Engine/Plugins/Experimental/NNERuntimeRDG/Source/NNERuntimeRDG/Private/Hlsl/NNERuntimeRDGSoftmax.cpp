// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGSoftmax.h"

#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersReduceCS.h"
#include "NNEHlslShadersSoftmaxCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorSoftmax, TEXT("NNE.Operator.Hlsl.Softmax"));

	/**
	 * Softmax operator implementation
	 */
	template< NNEHlslShaders::Internal::ESoftmaxOperatorType SoftmaxOperatorType, int Version >
	class FSoftmax : public FOperatorHlsl
	{

	public:

		FSoftmax() = default;
		virtual ~FSoftmax() = default;

	private:

		int32 Axis = 1;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const NNE::FTensorShape& InputShape = InputTensors[0]->GetShape();

			OutputTensors[0]->SetShape(InputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			const int32 InputDimensions = InputTensorDescs[0].GetShape().Rank();

			if (Version <= 11 && InputDimensions < 2)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Softmax: Input tensor should be at least 2-D (but got rank %d)", InputDimensions);
				return false;
			}

			if (InputTensorDescs[0].GetShape().Rank() != OutputTensorDescs[0].GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Softwax: Output should have the same rank as the input.");
				return false;
			}

			const int32 AxisLow = -InputDimensions;
			const int32 AxisHigh = InputDimensions - 1;

			Axis = Attributes.GetValueOrDefault<int32>(TEXT("axis"), Version <= 11 ? 1 : -1);
			if (Axis < AxisLow || AxisHigh < Axis)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Softmax: Invalid axis (should in the interval [%d, %d], but got %d)", AxisLow, AxisHigh, Axis);
				return false;
			}
			if(Axis < 0)
			{
				Axis += InputDimensions;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != FTensorRDGRef{});
			check(OutputTensors[0] != FTensorRDGRef{});

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			check(Input.GetVolume() == Output.GetVolume());

			const NNE::FTensorShape& InputShape = Input.GetShape();

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorSoftmax, "NNE.Operator.Hlsl.Softmax");

			// First apply Reduction(exp(x)) to temp buffer
			TReduceCS::FParameters* ReduceParameters = GraphBuilder.AllocParameters<TReduceCS::FParameters>();
			TReduceCS::FillInParameters(InputShape.GetData(), Axis, ReduceParameters);
			if(Version <= 11)
			{
				ReduceParameters->AxisSize *= ReduceParameters->NumElemAfterAxis;// Softmax flatten the input tensor to a 2D one
				ReduceParameters->NumElemAfterAxis = 1;
			}
			const FRDGBufferDesc SumExpBufferDesc = FRDGBufferDesc::CreateBufferDesc(
				Output.GetElementByteSize(), 
				Version <= 11 ?
					ReduceParameters->NumElemBeforeAxis
					:
					ReduceParameters->NumElemBeforeAxis * ReduceParameters->NumElemAfterAxis
				);

			FRDGBufferRef SumExpBuffer = GraphBuilder.CreateBuffer(SumExpBufferDesc, TEXT("NNE.Operator.Hlsl.Softmax.TempBuffer"), ERDGBufferFlags::None);

			TReduceCS::EnqueueRDG(GraphBuilder, ReduceParameters, Input.GetBuffer(), SumExpBuffer, EReduceOperatorType::SumExp);

			//Then Softmax
			const int32 NumElements = Input.GetVolume();
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(NumElements, FSoftmaxConstants::NUM_GROUP_THREADS);
			TSoftmaxCS::FParameters* SoftmaxParameters = GraphBuilder.AllocParameters<TSoftmaxCS::FParameters>();
			SoftmaxParameters->AxisSize = ReduceParameters->AxisSize;
			if(Version >= 13)
			{
				SoftmaxParameters->AfterAxisSize = ReduceParameters->NumElemAfterAxis;
			}
			SoftmaxParameters->Num = NumElements;
			SoftmaxParameters->ThreadCountX = ThreadGroupCount.X * FSoftmaxConstants::NUM_GROUP_THREADS;
			SoftmaxParameters->Input = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			SoftmaxParameters->InputSumExp = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(SumExpBuffer, PF_R32_FLOAT));
			SoftmaxParameters->Output = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
			TSoftmaxCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<TSoftmaxCS::FSoftmaxType>(SoftmaxOperatorType);
			PermutationVector.Set<TSoftmaxCS::FSingleDimension>(Version >= 13);
			TShaderMapRef<TSoftmaxCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Softmax.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				SoftmaxParameters,
				ThreadGroupCount);
		}
	};

	bool ValidateSoftmaxOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		// This matches versions 1, 11, 13 of the Softmax and LogSoftmax operators
		// https://github.com/onnx/onnx/blob/main/docs/Changelog.md#Softmax-1
		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	template<NNEHlslShaders::Internal::ESoftmaxOperatorType SoftmaxOperatorType, int Version>
	FOperatorHlsl* CreateSoftmaxOperator()
	{
		return new FSoftmax<SoftmaxOperatorType, Version>();
	}

	bool RegisterSoftmaxOperator(FOperatorRegistryHlsl& Registry)
	{
		// Adds specified version of Softmax and LogSoftmax to the Registry.
		#define OPS(Version) \
			Registry.OpAdd({ {TEXT("Softmax"), TEXT("Onnx")}, Version }, CreateSoftmaxOperator<NNEHlslShaders::Internal::ESoftmaxOperatorType::SOFTMAX, Version>, ValidateSoftmaxOperator); \
			Registry.OpAdd({ {TEXT("LogSoftmax"), TEXT("Onnx")}, Version }, CreateSoftmaxOperator<NNEHlslShaders::Internal::ESoftmaxOperatorType::LOG_SOFTMAX, Version>, ValidateSoftmaxOperator);

		OPS(1)
		OPS(11)
		OPS(13)

		#undef OPS
		
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
