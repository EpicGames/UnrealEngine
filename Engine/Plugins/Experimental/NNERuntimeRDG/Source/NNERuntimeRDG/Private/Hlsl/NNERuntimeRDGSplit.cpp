// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGSplit.h"

#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersSplitCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"
#include "Algo/Accumulate.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorSplit, TEXT("NNE.Operator.Hlsl.Split"));

	
	/**
	 * Split operator implementation
	 */
	template<int32 Version>
	class FSplitOperator : public FOperatorHlsl
	{
	public:

		FSplitOperator() {}
		virtual ~FSplitOperator() = default;
	private:
		int32 Axis = 0;
        TArray<int64> Splits;

		bool CheckSplitsSum(const NNE::FTensorShape& InputShape)
		{
			int64 SumToCheck = Algo::Accumulate(Splits, /*Init*/ 0);
			return InputShape.GetData()[Axis] == (uint32) SumToCheck;
		}

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 1 && InputTensors.Num() <= 2);
			check(OutputTensors.Num() >= 1);

			const NNE::FTensorShape& InputShape = InputTensors[0]->GetShape();

			if(Version >= 13 && InputTensors.Num() == 2)
			{
				const FTensor& SplitTensor = *InputTensors[1];
				if(!SplitTensor.HasPreparedData())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: split tensor need to be CPU-constant in order to compute output shapes.");
					return -1;
				}
				
				Splits = SplitTensor.GetPreparedData<int64>();
			}

			// Default: equal split
			if(Splits.IsEmpty())
			{
				Splits.Init(InputShape.GetData()[Axis] / OutputTensors.Num(), OutputTensors.Num());
				Splits.Last() += InputShape.GetData()[Axis] % OutputTensors.Num();
			}

			if(!CheckSplitsSum(InputShape))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: Sum of split values not equal to split axis' dimension.");
				return -1;
			}

			if(Splits.Num() > FSplitConstants::MAX_NUM_SPLITS)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: Number of splits (%d) exceeds maximum allowed (%d).", Splits.Num(), FSplitConstants::MAX_NUM_SPLITS);
				return -1;
			}
			
			if(OutputTensors.Num() != Splits.Num())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: Number of output tensors differs from number of splits provided.");
				return -1;
			}

			for(int32 Idx = 0; Idx < OutputTensors.Num(); Idx++)
			{
				TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> OutputShape {InputShape.GetData()};
				OutputShape[Axis] = (uint32) Splits[Idx];
				OutputTensors[Idx]->SetShape(NNE::FTensorShape::Make(OutputShape));
			}

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			
			check(InputTensorDescs.Num() >= 1 && InputTensorDescs.Num() <= 2);

			const int32 InputRank = InputTensorDescs[0].GetShape().Rank();
			Axis = Attributes.GetValueOrDefault<int32>(TEXT("axis"), Axis);
			if (Axis > InputRank || Axis  < -InputRank)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: Attribute 'Axis' should be in the range [-r,r] with r being the rank of the input (name: %ls) however got %d while rank is %d.", *InputTensorDescs[0].GetName(), Axis, InputRank);
				return false;
			}
			// Canonicalize Axis
			if (Axis < 0)
			{
				Axis += InputRank;
			}

			if constexpr (Version >= 18)
			{
				check( ((bool) Attributes.GetAttributeValue(TEXT("num_outputs"))) != (InputTensorDescs.Num() == 2) );
			}

			if(Version >= 18 && Attributes.GetAttributeValue(TEXT("num_outputs")))
			{
				const int32 NumOutputs = Attributes.GetValue<int32>(TEXT("num_outputs"));
				if(OutputTensorDescs.Num() != NumOutputs)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Split: Attribute 'num_outputs' doesn't match number of output tensors. Value: %d. Number of output tensors: %d.", NumOutputs, OutputTensorDescs.Num());
					return false;
				}
			}

			if(Version < 13 && Attributes.GetAttributeValue(TEXT("split")))
			{
				for(const int32& Length : Attributes.GetValue<TArray<int32>>(TEXT("split")))
				{
					Splits.Add((int64) Length);
				}
			}

			return true;
		}


		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() >= 1 && InputTensors.Num() <= 2);
			check(OutputTensors.Num() == Splits.Num());

			// Set shader parameters
			FSplitCS::FParameters* Params = GraphBuilder.AllocParameters<FSplitCS::FParameters>();
			const FTensorRDG& Input = *InputTensors[0];
			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			Params->Input = InputSRV;
			FillTensorStrideShaderParameters(Input, Params->InputTensorInfo, /*Idx*/ 0);
			FillTensorSizeShaderParameters(Input, Params->InputTensorInfo, /*Idx*/ 1);
			for(int32 TensorIdx = 0; TensorIdx < OutputTensors.Num(); TensorIdx++)
			{
				const FTensorRDG& Output = *OutputTensors[TensorIdx];
				const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
				Params->Output[TensorIdx] = OutputUAV;
				
				FTensorInfoParamArraySpan TensorInfoParamArraySpan {
					/*ArrayAtFunction*/ 		[&Params](uint32 Idx) -> FUintVector4& { return Params->OutputTensorInfo[Idx]; },
					/*Offset*/ 					TensorIdx * FSplitConstants::MAX_NUM_DIMENSIONS
				};

				FillTensorStrideShaderParameters(Output, TensorInfoParamArraySpan, /*Idx*/ 0);
				FillTensorSizeShaderParameters(Output, TensorInfoParamArraySpan, /*Idx*/ 1);
			}
			Params->Num = Input.GetVolume();

			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Input.GetVolume(), FSplitConstants::NUM_GROUP_THREADS);
			Params->ThreadCountX = ThreadGroupCount.X * FSplitConstants::NUM_GROUP_THREADS;

			FSplitCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSplitCS::FSplitRank>((int32) InputTensors[0]->GetShape().Rank());
			PermutationVector.Set<FSplitCS::FSplitAxis>(Axis);
			PermutationVector.Set<FSplitCS::FSplitNumSplits>(Splits.Num());
			

			TShaderMapRef<FSplitCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
        
            RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorSplit, "NNE.Operator.Hlsl.Split");

            FComputeShaderUtils::AddPass(
                GraphBuilder,
                RDG_EVENT_NAME("NNE.Operator.Hlsl.Split.Dispatch"),
                ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
                ComputeShader,
                Params,
                ThreadGroupCount);
			
		}
	};

	template<int32 Version>
	bool ValidateSplitOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("axis"), ENNERuntimeRDGDataAttributeDataType::Int32);
		if constexpr (Version < 13)
		{
			AttributeValidator.AddOptional(TEXT("split"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		}
		if constexpr (Version >= 18)
		{
			AttributeValidator.AddOptional(TEXT("num_outputs"), ENNERuntimeRDGDataAttributeDataType::Int32);
		}
        bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.SetTemplateCount(2);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64, /* TemplateIdx = */ 1 );
		InputValidator.AddRequired();
		if constexpr (Version >= 13)
		{
			InputValidator.AddOptional(/* TemplateIdx = */ 1);
		}
		bIsValid &= InputValidator.Validate(InputTypes);

		if constexpr (Version >= 18)
		{
			bIsValid &= ((bool) AttributeMap.GetAttributeValue(TEXT("num_outputs"))) != (InputTypes.Num() == 2);
		}

		return bIsValid;
	}

	template<int32 Version>
	FOperatorHlsl* CreateSplitOperator()
	{
		return new FSplitOperator<Version>();
	}


	bool RegisterSplitOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		#define REG_SPLIT_VER(Version) \
		Registry.OpAdd({{TEXT("Split"), TEXT("Onnx")}, Version}, CreateSplitOperator<Version>, ValidateSplitOperator<Version>);

		REG_SPLIT_VER(2);
		REG_SPLIT_VER(11);
		REG_SPLIT_VER(13);
		REG_SPLIT_VER(18);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
