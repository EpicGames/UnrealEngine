// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGSqueeze.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorSqueeze, TEXT("NNE.Operator.Hlsl.Squeeze"));

	/**
	 * Squeeze operator implementation
	 */
	class FSqueeze : public FOperatorHlsl
	{
	public:

		FSqueeze() {}
		virtual ~FSqueeze() = default;

		TArray<int32> Axes;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			TArray<uint32> OutputShapeData(X.GetShape().GetData());

			for (int32 Idx = Axes.Num() - 1; Idx >= 0; --Idx)
			{
				const int32 Axe = Axes[Idx];

				if (OutputShapeData[Axe] != 1)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Squeeze: Axe %d for 'Data' (name: %ls) should be targeting a dimension of size 1 but it is %d.", Axe, *X.GetName(), OutputShapeData[Axe]);
					return -1;
				}

				OutputShapeData.RemoveAt(Axe);
			}
			
			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			
			OutputTensors[0]->SetShape(OutputShape);
			if (X.HasPreparedData())
			{
				OutputTensors[0]->SetPreparedData<uint8>(X.GetPreparedData<uint8>());
			}
			
			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			Axes = Attributes.GetValue<TArray<int32>>(TEXT("axes"));
			for (int32 Axe : Axes)
			{
				if (Axe < 0)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Squeeze: Does not support negative axes");
					return false;
				}
				if (Axe >= InputTensorDescs[0].GetShape().Rank())
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Squeeze: Does not support axes greater than the number of dimensions of the input tensor shape");
					return false;
				}
			}

			Axes.Sort();
			
			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);
			
			const FTensorRDG& Data = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorSqueeze, "NNE.Operator.Hlsl.Squeeze");

			AddCopyBufferPass(GraphBuilder, Output.GetBuffer(), Data.GetBuffer());
		}
	};

	bool ValidateSqueezeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputSqueezes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddRequired(TEXT("axes"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Half);
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddSupportedType(ENNETensorDataType::Double);
		InputValidator.AddSupportedType(ENNETensorDataType::Int8);
		InputValidator.AddSupportedType(ENNETensorDataType::Int16);
		InputValidator.AddSupportedType(ENNETensorDataType::Int32);
		InputValidator.AddSupportedType(ENNETensorDataType::Int64);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt8);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt16);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt32);
		InputValidator.AddSupportedType(ENNETensorDataType::UInt64);
		InputValidator.AddRequired();

		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateSqueezeOperator()
	{
		return new FSqueeze();
	}

	bool RegisterSqueezeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Squeeze"), TEXT("Onnx")}, 1}, CreateSqueezeOperator, ValidateSqueezeOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
