// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGTranspose.h"

#include "Helper/NNERuntimeRDGHelperTranspose.h"
#include "NNEHlslShadersLog.h"
#include "NNEHlslShadersTransposeCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorTranspose, TEXT("NNE.Operator.Hlsl.Transpose"));

	/**
	 * Transpose operator implementation
	 */
	class FTranspose : public FOperatorHlsl
	{
	public:

		FTranspose() {}
		virtual ~FTranspose() = default;

	private:
		TArray<int32> Perm;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);

			const FTensor& X = *InputTensors[0];
			TConstArrayView<uint32> InputShape = X.GetShape().GetData();
			TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> OutputShape;

			OutputShape.SetNumUninitialized(InputShape.Num());
			for (int32 i = 0; i < InputShape.Num(); ++i)
			{
				OutputShape[i] = InputShape[Perm[i]];
			}
			OutputTensors[0]->SetShape(NNE::FTensorShape::Make(OutputShape));

			CPUHelper::Transpose::Apply(*InputTensors[0], Perm, *OutputTensors[0]);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 1);
			check(OutputTensorDescs.Num() == 1);

			const NNE::FTensorDesc& InputDesc = InputTensorDescs[0];
			const int32 InputRank = InputDesc.GetShape().Rank();
			TArray<int32> PermutationDefault;

			PermutationDefault.Reserve(NNE::FTensorShape::MaxRank);
			for (int32 i = 0; i < InputRank; ++i)
			{
				PermutationDefault.Add((InputRank -1) - i);
			}
			Perm = Attributes.GetValueOrDefault<TArray<int32>>(TEXT("perm"), PermutationDefault);

			if (Perm.Num() != InputTensorDescs[0].GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Transpose: 'perm' attribute should contain the same amount of element as the rank of the input tensor.");
				return false;
			}

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;

			check(InputTensors.Num() == 1);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(OutputTensors[0] != nullptr);

			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Output = *OutputTensors[0];
			const FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			const FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));
			const FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FTransposeConstants::NUM_GROUP_THREADS);

			// Set parameters
			FTransposeCS::FParameters* Params = GraphBuilder.AllocParameters<FTransposeCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FTransposeConstants::NUM_GROUP_THREADS;
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 0);
			FillTensorStrideShaderParameters(Input, Params->TensorInfo, 1);
			static_assert(NNE::FTensorShape::MaxRank <= NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS);
			check(Perm.Num() == Input.GetShape().Rank());
			for (int32 i = 0; i < NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS; ++i)
			{
				if (i < Input.GetShape().Rank())
				{
					Params->TensorInfo[i][2] = Params->TensorInfo[Perm[i]][1];
				}
				else
				{
					Params->TensorInfo[i][2] = 0;
				}
			}

			FTransposeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FTransposeCS::FTransposeNumDimensions>(Output.GetShape().Rank());

			TShaderMapRef<FTransposeCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorTranspose, "NNE.Operator.Hlsl.Transpose");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Transpose.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateTransposeOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputTransposes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(TEXT("perm"), ENNERuntimeRDGDataAttributeDataType::Int32Array);
		bIsValid &= AttributeValidator.Validate(AttributeMap);

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateTransposeOperator()
	{
		return new FTranspose();
	}

	bool RegisterTransposeOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Transpose"), TEXT("Onnx")}, 1}, CreateTransposeOperator, ValidateTransposeOperator);
		Registry.OpAdd({{TEXT("Transpose"), TEXT("Onnx")}, 13}, CreateTransposeOperator, ValidateTransposeOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
