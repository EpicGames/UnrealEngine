// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUpsample.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGAttributes.h"
#include "NNEHlslShadersUpsampleCS.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	DECLARE_GPU_STAT_NAMED(FNNEOperatorUpsample, TEXT("NNE.Operator.Hlsl.Upsample"));

	/**
	 * Upsample operator implementation
	 */
	class FUpsample : public FOperatorHlsl
	{
	public:

		FUpsample() {}
		virtual ~FUpsample() = default;

	private:

		FString Mode = AttrValue::Nearest;

	public:

		virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) override
		{
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);

			const FTensor& Input = *InputTensors[0];
			const FTensor& Scales = *InputTensors[1];
			
			if (!Scales.HasPreparedData())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Upsample: Input 'Scale' (name: %ls) should be constant for shape inference to succeed, however it is not constant.", *Scales.GetName());
				return -1;
			}

			TConstArrayView<float> ScalesData = Scales.GetPreparedData<float>();

			if (ScalesData.Num() != Input.GetShape().Rank())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Upsample: Input 'Scale' (name: %ls) have %d elements. While it should be the same as the rank of input 'X' (name : %ls) witch is %d", *Scales.GetName(), ScalesData.Num(), *Input.GetName(), Input.GetShape().Rank());
				return -1;
			}

			for (int32 i = 0; i < ScalesData.Num(); i++)
			{
				if (ScalesData[i] < 1.0f)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Upsample: Input 'Scale' takes values greater than or equal to 1.");
					return -1;
				}

				if (Mode.Equals(AttrValue::Linear) && i < ScalesData.Num() - 3 && ScalesData[i] > 1.0f)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Upsample: 'Linear mode' only support up to trilinear interpolation, meaning input 'Scale' values for the outermost dimensions (Rank - 3) are 1.");
					return -1;
				}
			}

			TArray<uint32> OutputShapeData;
			for (int32 i = 0; i < Input.GetShape().Rank(); ++i)
			{
				
				OutputShapeData.Emplace(FMath::FloorToInt32(Input.GetShape().GetData()[i] * ScalesData[i]));
			}

			NNE::FTensorShape OutputShape = NNE::FTensorShape::Make(OutputShapeData);
			OutputTensors[0]->SetShape(OutputShape);

			return 0;
		};

		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) override
		{
			check(InputTensorDescs.Num() == 2);
			check(OutputTensorDescs.Num() == 1);

			Mode = Attributes.GetValueOrDefault<FString>(AttrName::Mode, Mode);

			return true;
		}

		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InputTensors, TConstArrayView<FTensorRDGRef> OutputTensors) override
		{
			using namespace UE::NNEHlslShaders::Internal;
			
			check(InputTensors.Num() == 2);
			check(OutputTensors.Num() == 1);
			check(InputTensors[0] != nullptr);
			check(InputTensors[1] != nullptr);
			check(OutputTensors[0] != nullptr);
			const FTensorRDG& Input = *InputTensors[0];
			const FTensorRDG& Scales = *InputTensors[1];
			const FTensorRDG& Output = *OutputTensors[0];

			check(Scales.HasPreparedData());
			TConstArrayView<float> ScalesData = Scales.GetPreparedData<float>();

			FRDGBufferSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(Input.GetBuffer(), PF_R32_FLOAT));
			FRDGBufferUAVRef OutputUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.GetBuffer(), PF_R32_FLOAT));

			FIntVector ThreadGroupCount = ComputeElementWiseThreadGroups(Output.GetVolume(), FUpsampleConstants::NUM_GROUP_THREADS);

			// Set parameters
			FUpsampleCS::FParameters* Params = GraphBuilder.AllocParameters<FUpsampleCS::FParameters>();
			Params->Input = InputSRV;
			Params->Output = OutputUAV;
			FillTensorStrideShaderParameters(Input, Params->TensorInfo, 0);
			FillTensorStrideShaderParameters(Output, Params->TensorInfo, 1);
			FillTensorSizeShaderParameters(Input, Params->TensorInfo, 2);
			for (int32 i = 0; i < Input.GetShape().Rank(); ++i)
			{
				Params->TensorInfo[i][3] = FMath::CeilToInt32(ScalesData[i]);
			}
			Params->Num = Output.GetVolume();
			Params->ThreadCountX = ThreadGroupCount.X * FUpsampleConstants::NUM_GROUP_THREADS;

			const EUpsampleMode UpsampleMode = Mode.Equals(AttrValue::Linear) ?
				((ScalesData.Num() < 3 || ScalesData[ScalesData.Num() - 3] == 1.0f) ? EUpsampleMode::Bilinear : EUpsampleMode::Trilinear) :
				EUpsampleMode::Nearest;

			FUpsampleCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FUpsampleCS::FUpsampleNumDimensions>(Output.GetShape().Rank());
			PermutationVector.Set<FUpsampleCS::FUpsampleMode>(UpsampleMode);

			TShaderMapRef<FUpsampleCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			RDG_EVENT_SCOPE_STAT(GraphBuilder, FNNEOperatorUpsample, "NNE.Operator.Hlsl.Upsample");

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NNE.Operator.Hlsl.Upsample.Dispatch"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				Params,
				ThreadGroupCount);
		}
	};

	bool ValidateUpsampleOperator(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		bool bIsValid = true;

		FAttributeValidator AttributeValidator;
		AttributeValidator.AddOptional(AttrName::Mode, ENNERuntimeRDGDataAttributeDataType::String);
		bIsValid &= AttributeValidator.Validate(AttributeMap);
		
		FString Mode = AttributeMap.GetValueOrDefault<FString>(AttrName::Mode, AttrValue::Nearest);
		if (!Mode.Equals(AttrValue::Nearest) && !Mode.Equals(AttrValue::Linear))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Upsample: Only supports %ls or %ls as value for attribute %ls", AttrValue::Nearest, AttrValue::Linear, AttrName::Mode);
			return false;
		}

		FInputValidator InputValidator;
		InputValidator.AddSupportedType(ENNETensorDataType::Float);
		InputValidator.AddRequired();
		InputValidator.AddRequired();
		bIsValid &= InputValidator.Validate(InputTypes);

		return bIsValid;
	}

	FOperatorHlsl* CreateUpsampleOperator()
	{
		return new FUpsample();
	}

	bool RegisterUpsampleOperator(FOperatorRegistryHlsl& Registry)
	{
		// Note: support of a particular version is partial with respect to tensor data types (only the most typical ones are usually supported).
		Registry.OpAdd({{TEXT("Upsample"), TEXT("Onnx")}, 9}, CreateUpsampleOperator, ValidateUpsampleOperator);
		return true;
	}
} // UE::NNERuntimeRDG::Private::Hlsl
