// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::NNERuntimeRDG::Private
{

namespace AttrName
{
	static constexpr auto* Mode = TEXT("mode");
}

namespace AttrValue
{
	static constexpr auto* Nearest = TEXT("nearest");
	static constexpr auto* Linear = TEXT("linear");
}

} // UE::NNERuntimeRDG::Private