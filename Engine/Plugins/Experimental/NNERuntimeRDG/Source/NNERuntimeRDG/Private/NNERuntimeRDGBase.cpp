// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGBase.h"

#include "Helper/NNERuntimeRDGLogHelper.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"
#include "Serialization/MemoryReader.h"

namespace UE::NNERuntimeRDG::Private
{

//
//
//
bool AlwaysValidValidationFunction(
	const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, 
	TConstArrayView<ENNETensorDataType> InputTensorTypes,
	TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
{
	return true;
}


//
//
//
FInputValidator::FInputValidator() : 
	NumRequiredInput(0), NumOptionalInput(0)
{
	SetTemplateCount(1);
}

bool FInputValidator::Validate(TConstArrayView<ENNETensorDataType> InputTypes)
{
	check(InputTemplateIndices.Num() == NumRequiredInput + NumOptionalInput);

	bool bAreInputValid = true;
	int32 NumInputsToValidate = FMath::Min(InputTemplateIndices.Num(), InputTypes.Num());
	
	if (InputTypes.Num() < NumRequiredInput)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Required '%d' inputs but found '%d'.", NumRequiredInput, InputTypes.Num());
		bAreInputValid = false;
	}
	if (InputTypes.Num() > NumRequiredInput+NumOptionalInput)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Got a total of '%d' inputs but should have '%d' maximum.", InputTypes.Num(), NumRequiredInput + NumOptionalInput);
		bAreInputValid = false;
	}
	
	for (int32 Idx = 0; Idx < NumInputsToValidate; ++Idx)
	{
		const int32 TemplateIdx = InputTemplateIndices[Idx];
		
		check(TemplateIdx < TemplateTypes.Num());
		if (!TemplateTypes[TemplateIdx].Contains(InputTypes[Idx]) &&
			!(InputTypes[Idx] == ENNETensorDataType::None && Idx >= NumRequiredInput))
		{
			FString TargetType = LogHelper::GetTensorDataTypeName(InputTypes[Idx]);
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Input at index '%d' (from template T%d) is of type '%ls' witch is not supported for that input.", Idx, TemplateIdx, *TargetType);
			bAreInputValid = false;
		}
	}
	return bAreInputValid;
}
void FInputValidator::SetTemplateCount(int32 TemplateCount)
{
	TemplateTypes.Init(TSet<ENNETensorDataType>{}, TemplateCount);
}
void FInputValidator::AddSupportedType(ENNETensorDataType Type, int32 TemplateIdx)
{
	check(TemplateTypes.Num() > TemplateIdx);
	TemplateTypes[TemplateIdx].Add(Type);
}
void FInputValidator::AddOptional(int32 TemplateIdx)
{
	InputTemplateIndices.Add(TemplateIdx);
	++NumOptionalInput;
}

void FInputValidator::AddRequired(int32 TemplateIdx)
{
	checkf(NumOptionalInput==0, TEXT("All required attribute should be declared before the optional ones as they are referenced by indices"));
	InputTemplateIndices.Add(TemplateIdx);
	++NumRequiredInput;
}


//
//
//
void FAttributeValidator::AddOptional(const FString& Name, ENNERuntimeRDGDataAttributeDataType Type)
{
#if DO_CHECK
	const bool bIsFoundInOptional = (nullptr != OptionalAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; }));
	const bool bIsFoundInRequired = (nullptr != RequiredAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; }));
	checkf(!bIsFoundInOptional, TEXT("Attribute name should be unique"));
	checkf(!bIsFoundInRequired, TEXT("Attribute name should be unique"));
#endif
	OptionalAttributes.Emplace(Name, Type);
}

void FAttributeValidator::AddRequired(const FString& Name, ENNERuntimeRDGDataAttributeDataType Type)
{
#if DO_CHECK
	const bool bIsFoundInOptional = (nullptr != OptionalAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; }));
	const bool bIsFoundInRequired = (nullptr != RequiredAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; }));
	checkf(!bIsFoundInOptional, TEXT("Attribute name should be unique"));
	checkf(!bIsFoundInRequired, TEXT("Attribute name should be unique"));
#endif
	RequiredAttributes.Emplace(Name, Type);
}

bool FAttributeValidator::Validate(const NNERuntimeRDGData::Internal::FAttributeMap& AttributesToValidate)
{
	bool bAreAttributesValid = true;

	//Verify all required attribute are matching specifications
	for (int32 Idx = 0; Idx < RequiredAttributes.Num(); ++Idx)
	{
		const FNNERuntimeRDGDataAttributeValue* FoundAttribute = AttributesToValidate.GetAttributeValue(RequiredAttributes[Idx].Name);
		
		if (FoundAttribute == nullptr)
		{
			bAreAttributesValid = false;
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Required attribute '%ls' not found.",
				*RequiredAttributes[Idx].Name);
		}
		else if (RequiredAttributes[Idx].Type != FoundAttribute->GetType())
		{
			bAreAttributesValid = false;
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Required attribute '%ls' type '%d' does not match expected type '%d'.",
				*RequiredAttributes[Idx].Name,
				(int)FoundAttribute->GetType(),
				(int)RequiredAttributes[Idx].Type);
		}
	}

	//Verify all optional attribute are matching specifications
	for (int32 Idx = 0; Idx < OptionalAttributes.Num(); ++Idx)
	{
		const FNNERuntimeRDGDataAttributeValue* FoundAttribute = AttributesToValidate.GetAttributeValue(OptionalAttributes[Idx].Name);
		
		if ((FoundAttribute != nullptr) && (OptionalAttributes[Idx].Type != FoundAttribute->GetType()))
		{
			bAreAttributesValid = false;
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Optional attribute '%ls' type '%d' does not match expected type '%d'.",
				*OptionalAttributes[Idx].Name,
				(int)FoundAttribute->GetType(),
				(int)OptionalAttributes[Idx].Type);
		}
	}

	//Verify all attributes are either required or optional, otherwise they are unsupported
	for (int32 Idx = 0; Idx < AttributesToValidate.Num(); ++Idx)
	{
		const FString& Name = AttributesToValidate.GetName(Idx);
		const FEntry* OptionalAttribute = OptionalAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; });
		const FEntry* RequiredAttribute = RequiredAttributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; });
		
		if (OptionalAttribute == nullptr && RequiredAttribute == nullptr)
		{
			bAreAttributesValid = false;
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Found unsupported attribute '%ls'.", *Name);
		}
	}

	return bAreAttributesValid;
}

} // UE::NNERuntimeRDG::Private
