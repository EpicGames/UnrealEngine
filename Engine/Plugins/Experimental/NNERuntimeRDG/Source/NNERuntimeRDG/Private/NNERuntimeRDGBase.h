// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGDataAttributeValue.h"
#include "NNERuntimeRDGDataFormat.h"
#ifdef NNE_UTILITIES_AVAILABLE
#include "NNERuntimeRDGUtilsModelOptimizerInterface.h"
#endif //NNE_UTILITIES_AVAILABLE
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include "RenderGraphResources.h"
#include "Serialization/MemoryReader.h"
#include "ShaderParameterUtils.h"



BEGIN_SHADER_PARAMETER_STRUCT(FNNETensorReadbackParameters, )
	RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

class FRDGBuilder;

namespace UE::NNE { class FTensorDesc; }

namespace UE::NNERuntimeRDG::Private
{

/**
 * Interface for all operators to prepare the model tensors at scheduling time
 */
struct IPrepareOperator
{
	virtual ~IPrepareOperator() = default;
	virtual int PrepareOutputs(TConstArrayView<FTensorRef> InputTensors, TArrayView<FTensorRef> OutputTensors) = 0;
};

/**
* Interface for all ML operators running on the RDG 
*/
struct IOperatorRDG
{
	virtual ~IOperatorRDG() = default;
};

class FTensorRDG : public FTensor
{
	FRDGBufferRef Buffer{};
	
public:
	static FTensorRDG Make(const NNE::FTensorDesc& TensorDesc, const NNE::FTensorShape& Shape, FRDGBufferRef Buffer)
	{
		check(Shape.IsCompatibleWith(TensorDesc.GetShape()));
		FTensorRDG TensorRDG;
		TensorRDG.Buffer = nullptr;
		TensorRDG.Name = TensorDesc.GetName();
		TensorRDG.DataType = TensorDesc.GetDataType();
		TensorRDG.Shape = Shape;
		TensorRDG.Volume = Shape.Volume();
		TensorRDG.DataSize = (uint64)NNE::GetTensorDataTypeSizeInBytes(TensorRDG.DataType) * TensorRDG.Volume;
		TensorRDG.Buffer = Buffer;
		check(TensorRDG.Volume <= TNumericLimits<uint32>::Max());
		return TensorRDG;
	}

	bool IsValid() const { return HasBuffer() || GetVolume() == 0; }
	bool HasBuffer() const { return Buffer != FRDGBufferRef{}; }
	void SetBuffer(FRDGBufferRef Inbuffer){ Buffer = Inbuffer; }
	FRDGBufferRef GetBuffer() const { return Buffer; }
};

using FTensorRDGRef = FTensorRDG*;
using FTensorRDGArray = TArray<FTensorRDG, TInlineAllocator<16>>;
using FTensorRDGRefMap = TMap<int32, FTensorRDGRef>;
using FIntArray = TArray<int32, TInlineAllocator<16>>;

bool AlwaysValidValidationFunction(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes);

class FInputValidator
{
public:
	FInputValidator();
	void AddRequired(int32 TemplateIdx = 0);
	void AddOptional(int32 TemplateIdx = 0);
	void SetTemplateCount(int32 TemplateCount);
	void AddSupportedType(ENNETensorDataType Type, int32 TemplateIdx = 0);
	bool Validate(TConstArrayView<ENNETensorDataType> InputTypes);

private:
	TArray<TSet<ENNETensorDataType>, TInlineAllocator<16>> TemplateTypes;
	FIntArray InputTemplateIndices;
	int32 NumRequiredInput;
	int32 NumOptionalInput;
};

class FAttributeValidator
{
public:
	void AddOptional(const FString& Name, ENNERuntimeRDGDataAttributeDataType Type);
	void AddRequired(const FString& Name, ENNERuntimeRDGDataAttributeDataType Type);
	bool Validate(const NNERuntimeRDGData::Internal::FAttributeMap& AttributesToValidate);

private:
	struct FEntry
	{
		FEntry(const FString& InName, ENNERuntimeRDGDataAttributeDataType InType)
			: Name(InName), Type(InType)
		{
		}

		//Idea: we could extended as needed by operator to support more validation especially around the range of the values.
		//An example is ConvTranspose `auto_pad` enum style string that can only take a few values
		//In the same direction we might only support a range of value for a float (for example
		//we only support integer but the type is float, or only positive values for an int32)
		FString Name;
		ENNERuntimeRDGDataAttributeDataType Type;
	};

	TArray<FEntry> RequiredAttributes;
	TArray<FEntry> OptionalAttributes;
};

typedef uint32 TOperatorVersionType;

struct FOperatorDescUnversioned
{
	FString OpName;
	FString DomainName;

	bool operator==(FOperatorDescUnversioned const& Other) const
	{
		return OpName == Other.OpName && DomainName == Other.DomainName;
	}
};

FORCEINLINE uint32 GetTypeHash(const FOperatorDescUnversioned& OpDescUnversioned)
{
	uint32 Hash = GetTypeHash(OpDescUnversioned.OpName) ^ GetTypeHash(OpDescUnversioned.DomainName);
	return Hash;
}

struct FOperatorDesc : public FOperatorDescUnversioned
{
	TOptional<TOperatorVersionType> Version; // Unset means no versioning for the operator

	// Return the full name	in the format <DomainName>:<OpName>(:<Version>)
	FString GetFullName() const
	{
		return FString::Printf(TEXT("%s:%s"), *DomainName, *OpName) 
			+ (Version.IsSet() ? *FString::Printf(TEXT(":%d"), *Version) : TEXT(""));
	}
};

/**
 * Registry for RDG ML operators
 * 
 * Operators can be registered either as versioned or unversioned (FOperatorDesc::Version unset).
 * The same operator (op and domain name pair) can be registered for different versions, but there can only be 
 * one unversioned registration (and no versioned registration in such case).
 * 
 * When finding a registration via OpFind() and OpFindValidation()
 * 		- in case there's an unversioned registration of an operator: 
 * 		  any input OpDesc with the same (OpName, DomainName) pair will match the registration,
 * 		- in case there are versioned registrations of an operator: 
 * 		  the registration with same (OpName, DomainName) pair and highest version (but <= than input OpDesc.Version) 
 * 		  will match the input OpDesc.
 * 
 */
template<class TOperatorType>
class TOperatorRegistryRDG
{
public:

	typedef TOperatorType* (*OperatorCreateFunc)();
	typedef bool (*OperatorValidateFunc)(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes);

	static TOperatorRegistryRDG* Get()
	{
		static TOperatorRegistryRDG Inst;

		return &Inst;
	}

	OperatorValidateFunc OpFindValidation(const FOperatorDesc& OpDesc)
	{
		const FOperatorFunctions* OperatorFunctions = OpFindFunctions(OpDesc);
		if(OperatorFunctions == nullptr)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator: %ls is not registered", *OpDesc.GetFullName());
			return nullptr;
		}

		return OperatorFunctions->ValidateFunc;
	}

	OperatorCreateFunc OpFind(const FOperatorDesc& OpDesc)
	{
		const FOperatorFunctions* OperatorFunctions = OpFindFunctions(OpDesc);
		if(OperatorFunctions == nullptr)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator: %ls is not registered", *OpDesc.GetFullName());
			return nullptr;
		}

		return OperatorFunctions->CreateFunc;
	}

	bool OpAdd(const FOperatorDesc& OpDesc, OperatorCreateFunc CreateFunc, OperatorValidateFunc ValidateFunc = AlwaysValidValidationFunction)
	{
		TOperatorVersionToFunctionsMap* VersionToFunctions = Operators.Find(OpDesc);
		if (VersionToFunctions != nullptr)
		{
			const FOperatorFunctions* OperatorFunctions = VersionToFunctions->Find(OpDesc.Version);
			if(OperatorFunctions != nullptr)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator is already registered: %ls", *OpDesc.GetFullName());
				return false;
			}
			else if(!OpDesc.Version.IsSet() || VersionToFunctions->Find(TOptional<TOperatorVersionType>()) != nullptr)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator %ls is unversioned, can't register a version of it", *FOperatorDesc{{OpDesc}}.GetFullName());
				return false;
			}
			VersionToFunctions->Add(OpDesc.Version, FOperatorFunctions{CreateFunc, ValidateFunc});
		}
		else
		{
			Operators.Add(OpDesc, TOperatorVersionToFunctionsMap { 
					{ OpDesc.Version, {CreateFunc, ValidateFunc} } 
				});
		}
		
		return true;
	}

	FString ListAllRegisteredOperators() const
	{
		class FLoggableOperatorDesc
		{
			FString OpDomainAndName;
			FString SupportedOpsets;

		public:

			FLoggableOperatorDesc(const FOperatorDescUnversioned& InOpDesc, TArrayView<TOptional<TOperatorVersionType>> InOpVersions)
			{
				OpDomainAndName = FString::Printf(TEXT("%s,%s"), *InOpDesc.DomainName, *InOpDesc.OpName);

				TArray<TOptional<TOperatorVersionType>, TInlineAllocator<8>> OpVersions(InOpVersions);

				OpVersions.Sort([](const TOptional<TOperatorVersionType>& lhs, const TOptional<TOperatorVersionType>& rhs)
					{
						if (!lhs.IsSet())
						{
							return true;
						}
						else if (!rhs.IsSet())
						{
							return false;
						}
						else
						{
							return *lhs < *rhs;
						}
					});

				for (const TOptional<TOperatorVersionType>& OpVersion : OpVersions)
				{
					SupportedOpsets += (OpVersion.IsSet() ? FString::Printf(TEXT(",%d"), *OpVersion) : TEXT(",Unversioned"));
				}
			}

			const FString& GetDomainAndName() const
			{
				return OpDomainAndName;
			}
			const FString& GetSupportedOpset() const
			{
				return SupportedOpsets;
			}
		};

		TArray<FLoggableOperatorDesc> LoggableOperatorDesc;
		TArray<TOptional<TOperatorVersionType>> Versions;

		for (const TPair<FOperatorDescUnversioned, TOperatorVersionToFunctionsMap>& Pair : Operators)
		{
			Pair.Value.GenerateKeyArray(Versions);
			LoggableOperatorDesc.Emplace(Pair.Key, Versions);
		}

		LoggableOperatorDesc.Sort([](const FLoggableOperatorDesc& lhs, const FLoggableOperatorDesc& rhs) 
		{
			return lhs.GetDomainAndName() < rhs.GetDomainAndName();
		});

		FString OpListWithOpset;
		for (const FLoggableOperatorDesc& LoggableOp : LoggableOperatorDesc)
		{
			OpListWithOpset += LoggableOp.GetDomainAndName();
			OpListWithOpset += LoggableOp.GetSupportedOpset();
			OpListWithOpset += TEXT("\n");
		}

		return OpListWithOpset;
	}

private:

	struct FOperatorFunctions
	{
		OperatorCreateFunc 		CreateFunc;
		OperatorValidateFunc 	ValidateFunc;
	};

	const FOperatorFunctions* OpFindFunctions(const FOperatorDesc& OpDesc) const
	{
		const TOperatorVersionToFunctionsMap* VersionToFunctions = Operators.Find(OpDesc);
		if (VersionToFunctions != nullptr)
		{
			if(VersionToFunctions->Contains(OpDesc.Version))
			{
				return &(*VersionToFunctions)[OpDesc.Version];
			}
			if(VersionToFunctions->Contains(TOptional<TOperatorVersionType>()))
			{
				return &(*VersionToFunctions)[TOptional<TOperatorVersionType>()];
			}
		}

		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator: %ls is not registered", *OpDesc.GetFullName());
		return nullptr;
	}


	typedef TMap<TOptional<TOperatorVersionType>, FOperatorFunctions> TOperatorVersionToFunctionsMap;

	TMap<FOperatorDescUnversioned, TOperatorVersionToFunctionsMap> Operators;
};

#ifdef NNE_UTILITIES_AVAILABLE
/**
 * Validator for RDG ML operators
 */
template<class TOperatorType>
class TModelValidatorRDG : public NNERuntimeRDGUtils::Internal::IModelValidator
{
public:
	virtual FString GetName() const 
	{
		return TEXT("RDG Model validator");
	}
	
	virtual bool ValidateModel(TConstArrayView<uint8> InputModel) const override
	{
		FNNERuntimeRDGDataModelFormat	Format;

		FMemoryReaderView Reader(InputModel, /*bIsPersitent =*/ true);

		Format.Serialize(Reader);

		TOperatorRegistryRDG<TOperatorType>* Registry = TOperatorRegistryRDG<TOperatorType>::Get();
		check(Registry != nullptr);

		for (int32 Idx = 0; Idx < Format.Operators.Num(); ++Idx)
		{
			TArray<ENNETensorDataType> InputTensorTypes;
			TArray<NNE::FSymbolicTensorShape> InputTensorShapes;
			NNERuntimeRDGData::Internal::FAttributeMap AttributeMap;
			
			for (int32 InputTensorIndex: Format.Operators[Idx].InTensors)
			{
				InputTensorTypes.Add(Format.Tensors[InputTensorIndex].DataType);
				InputTensorShapes.Add(NNE::FSymbolicTensorShape::Make(Format.Tensors[InputTensorIndex].Shape));
			}
			for (const FNNERuntimeRDGDataAttributeDesc& Desc : Format.Operators[Idx].Attributes)
			{
				AttributeMap.SetAttribute(Desc.Name, Desc.Value);
			}

			const FString& OpType = Format.Operators[Idx].TypeName;
			
			typename TOperatorRegistryRDG<TOperatorType>::OperatorValidateFunc ValidationFn = Registry->OpFindValidation({{OpType, Format.Operators[Idx].DomainName}, Format.Operators[Idx].Version});

			if (!ValidationFn)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "OperatorRegistry failed to find validation for operator:%ls", *OpType);
				return false;
			}
			
			if (!ValidationFn(AttributeMap, InputTensorTypes, InputTensorShapes))
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "OperatorRegistry failed to validate operator:%ls", *OpType);
				return false;
			}
		}

		return true;
	}
};
#endif //NNE_UTILITIES_AVAILABLE

} // UE::NNERuntimeRDG::Private