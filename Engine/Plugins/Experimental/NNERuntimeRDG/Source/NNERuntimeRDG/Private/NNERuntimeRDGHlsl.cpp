// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHlsl.h"

#include "Misc/SecureHash.h"
#include "NNEHlslShadersLog.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "NNERuntimeRDGHlslHelper.h"
#include "NNERuntimeRDGModelHlsl.h"
#ifdef NNE_UTILITIES_AVAILABLE
#include "NNERuntimeRDGUtilsModelOptimizerInterface.h"
#include "NNERuntimeRDGUtilsModelOptimizer.h"
#endif // NNE_UTILITIES_AVAILABLE
#include "HAL/IConsoleManager.h"
#include "Hlsl/NNERuntimeRDGBatchNormalization.h"
#include "Hlsl/NNERuntimeRDGCast.h"
#include "Hlsl/NNERuntimeRDGConv.h"
#include "Hlsl/NNERuntimeRDGConcat.h"
#include "Hlsl/NNERuntimeRDGConstant.h"
#include "Hlsl/NNERuntimeRDGConvTranspose.h"
#include "Hlsl/NNERuntimeRDGCumSum.h"
#include "Hlsl/NNERuntimeRDGDepthToSpace.h"
#include "Hlsl/NNERuntimeRDGDropout.h"
#include "Hlsl/NNERuntimeRDGElementWiseBinary.h"
#include "Hlsl/NNERuntimeRDGElementWiseUnary.h"
#include "Hlsl/NNERuntimeRDGElementWiseVariadic.h"
#include "Hlsl/NNERuntimeRDGFlatten.h"
#include "Hlsl/NNERuntimeRDGGather.h"
#include "Hlsl/NNERuntimeRDGGemm.h"
#include "Hlsl/NNERuntimeRDGGlobalPool.h"
#include "Hlsl/NNERuntimeRDGIdentity.h"
#include "Hlsl/NNERuntimeRDGInstanceNormalization.h"
#include "Hlsl/NNERuntimeRDGLayerNormalization.h"
#include "Hlsl/NNERuntimeRDGGatherElements.h"
#include "Hlsl/NNERuntimeRDGPad.h"
#include "Hlsl/NNERuntimeRDGPool.h"
#include "Hlsl/NNERuntimeRDGReduce.h"
#include "Hlsl/NNERuntimeRDGResize.h"
#include "Hlsl/NNERuntimeRDGReshape.h"
#include "Hlsl/NNERuntimeRDGScatterND.h"
#include "Hlsl/NNERuntimeRDGShape.h"
#include "Hlsl/NNERuntimeRDGSize.h"
#include "Hlsl/NNERuntimeRDGSlice.h"
#include "Hlsl/NNERuntimeRDGSplit.h"
#include "Hlsl/NNERuntimeRDGSoftmax.h"
#include "Hlsl/NNERuntimeRDGSqueeze.h"
#include "Hlsl/NNERuntimeRDGTranspose.h"
#include "Hlsl/NNERuntimeRDGUnsqueeze.h"
#include "Hlsl/NNERuntimeRDGUpsample.h"
#include "Hlsl/NNERuntimeRDGMatMul.h"

using namespace UE::NNERuntimeRDG::Private::Hlsl;

FGuid UNNERuntimeRDGHlslImpl::GUID = FGuid((int32)'R', (int32)'D', (int32)'G', (int32)'H');
int32 UNNERuntimeRDGHlslImpl::Version = 0x00000007;

bool UNNERuntimeRDGHlslImpl::Init()
{
	FOperatorRegistryHlsl* Registry = FOperatorRegistryHlsl::Get();
	check(Registry != nullptr);

	RegisterBatchNormalizationOperator(*Registry);
	RegisterCastOperator(*Registry);
	RegisterConvOperator(*Registry);
	RegisterConcatOperator(*Registry);
	RegisterConstantOperator(*Registry);
	RegisterConvTransposeOperator(*Registry);
	RegisterCumSumOperator(*Registry);
	RegisterDepthToSpaceOperator(*Registry);
	RegisterDropoutOperator(*Registry);
	RegisterElementWiseBinaryOperators(*Registry);
	RegisterElementWiseUnaryOperators(*Registry);
	RegisterElementWiseVariadicOperators(*Registry);
	RegisterFlattenOperator(*Registry);
	RegisterGatherOperator(*Registry);
	RegisterGemmOperator(*Registry);
	RegisterGlobalPoolOperators(*Registry);
	RegisterIdentityOperator(*Registry);
	RegisterInstanceNormalizationOperator(*Registry);
	RegisterLayerNormalizationOperator(*Registry);
	RegisterGatherElementsOperator(*Registry);
	RegisterPadOperator(*Registry);
	RegisterPoolOperators(*Registry);
	RegisterReduceOperators(*Registry);
	RegisterReshapeOperator(*Registry);
	RegisterResizeOperator(*Registry);
	RegisterScatterNDOperator(*Registry);
	RegisterShapeOperator(*Registry);
	RegisterSizeOperator(*Registry);
	RegisterSliceOperator(*Registry);
	RegisterSplitOperator(*Registry);
	RegisterSoftmaxOperator(*Registry);
	RegisterSqueezeOperator(*Registry);
	RegisterTransposeOperator(*Registry);
	RegisterUnsqueezeOperator(*Registry);
	RegisterUpsampleOperator(*Registry);
	RegisterMatMulOperator(*Registry);

	return true;
}

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	namespace ConsoleCommands
	{
		static FAutoConsoleCommand GetAutomationRuntimeFilterCommand(
			TEXT("nne.hlsl.getoperatorsupportmatrix"), TEXT("Get the NNERuntimeRDGHlsl operators support matrix in term of ONNX."),
			FConsoleCommandWithArgsDelegate::CreateStatic(
				[](const TArray< FString >& Args)
				{
					FOperatorRegistryHlsl* Registry = FOperatorRegistryHlsl::Get();
					check(Registry != nullptr);
					FString SupportMatrix = Registry->ListAllRegisteredOperators();
					UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Operators support matrix: \n%ls", *SupportMatrix);
				}
			)
		);
	} // namespace ConsoleCommands

	namespace Details
	{
		UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus CheckCanCreateModelData(bool bShouldLog, const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
		{
#ifdef NNE_UTILITIES_AVAILABLE
			if (FileType.Compare("onnx", ESearchCase::IgnoreCase) != 0)
			{
				if (bShouldLog)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Cannot create the model data with id %ls (Filetype: %ls), Only 'onnx' file type is supported", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
				}
				return UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus::Fail;
			}

			// Check model is not > 2GB
			if ((TArray<uint8>::SizeType)FileData.Num() != FileData.Num())
			{
				if (bShouldLog)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Cannot create the model data with id %ls (Filetype: %ls), models > 2GBs are not supported", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
				}
				return UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus::Fail;
			}

			if (!AdditionalFileData.IsEmpty())
			{
				if (bShouldLog)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Cannot create the model data with id %ls (Filetype: %ls), external data not supported at the moment, please convert the model to internal storage. See https://onnx.ai/onnx/repo-docs/ExternalData.html", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
				}
				return UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus::Fail;
			}

			return UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus::Ok;
#else
			if (bShouldLog)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Cannot create the model data with id %ls (Filetype: %ls), NNERuntimeRDGUtils is not available on this platform", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
			}
			return UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus::Fail;
#endif
		}
	} // namespace Details

} // namespace UE::NNERuntimeRDG::Private::Hlsl


UNNERuntimeRDGHlslImpl::ECanCreateModelDataStatus UNNERuntimeRDGHlslImpl::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return Details::CheckCanCreateModelData(/*bShouldLog*/ false , FileType, FileData, AdditionalFileData, FileId, TargetPlatform);
}

UNNERuntimeRDGHlslImplRDG::ECanCreateModelRDGStatus UNNERuntimeRDGHlslImplRDG::CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const
{
	int32 GuidSize = sizeof(GUID);
	int32 VersionSize = sizeof(Version);
	TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName());

	if (!SharedData.IsValid())
	{
		return ECanCreateModelRDGStatus::Fail;
	}

	TConstArrayView64<uint8> Data = SharedData->GetView();

	if (Data.Num() <= GuidSize + VersionSize)
	{
		return ECanCreateModelRDGStatus::Fail;
	}
	bool bResult = FGenericPlatformMemory::Memcmp(&(Data[0]), &(GUID), GuidSize) == 0;
	bResult &= FGenericPlatformMemory::Memcmp(&(Data[GuidSize]), &(Version), VersionSize) == 0;

	return bResult ? ECanCreateModelRDGStatus::Ok : ECanCreateModelRDGStatus::Fail;
};

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeRDGHlslImpl::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	if (Details::CheckCanCreateModelData(/*bShouldLog*/ true, FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		return {};
	}

#ifdef NNE_UTILITIES_AVAILABLE
	TUniquePtr<UE::NNERuntimeRDGUtils::Internal::IModelOptimizer> Optimizer = UE::NNERuntimeRDGUtils::Internal::CreateModelOptimizer();
	Optimizer->AddValidator(MakeShared<UE::NNERuntimeRDG::Private::TModelValidatorRDG<FOperatorHlsl>>());

	TArray<uint8> OutputModel;
	if (!Optimizer->Optimize(FileData, OutputModel))
	{
		return {};
	}

	TArray<uint8> Result;
	FMemoryWriter Writer(Result);
	
	Writer << GUID;
	Writer << Version;
	Writer.Serialize(OutputModel.GetData(), OutputModel.Num());

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Result)), 0);
#else //NNE_UTILITIES_AVAILABLE
	return {};
#endif //NNE_UTILITIES_AVAILABLE
};

FString UNNERuntimeRDGHlslImpl::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return FileId.ToString(EGuidFormats::Digits) + "-" + UNNERuntimeRDGHlslImpl::GUID.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(UNNERuntimeRDGHlslImpl::Version);
}

TSharedPtr<UE::NNE::IModelRDG> UNNERuntimeRDGHlslImplRDG::CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData)
{
	if (CanCreateModelRDG(ModelData) != ECanCreateModelRDGStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Cannot create a model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelRDG>();
	}

	TSharedPtr<UE::NNE::FSharedModelData> Data = ModelData->GetModelData(GetRuntimeName());
	check(Data.IsValid());
	UE::NNERuntimeRDG::Private::Hlsl::FModel* Model = new UE::NNERuntimeRDG::Private::Hlsl::FModel(Data);

	return TSharedPtr<UE::NNE::IModelRDG>(Model);
}
