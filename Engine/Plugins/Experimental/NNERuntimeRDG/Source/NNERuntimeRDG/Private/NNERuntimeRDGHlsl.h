// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNERuntime.h"
#include "NNERuntimeRDG.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"

#include "NNERuntimeRDGHlsl.generated.h"

UCLASS()
class UNNERuntimeRDGHlslImpl : public UObject, public INNERuntime
{
	GENERATED_BODY()

public:
	static FGuid GUID;
	static int32 Version;

	UNNERuntimeRDGHlslImpl() {};
	virtual ~UNNERuntimeRDGHlslImpl() {}

	bool Init();

	virtual FString GetRuntimeName() const override { return TEXT("NNERuntimeRDGHlsl"); };

	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
};

UCLASS()
class UNNERuntimeRDGHlslImplRDG : public UNNERuntimeRDGHlslImpl, public INNERuntimeRDG
{
	GENERATED_BODY()

public:
	virtual ~UNNERuntimeRDGHlslImplRDG() {}

	virtual ECanCreateModelRDGStatus CanCreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelRDG> CreateModelRDG(const TObjectPtr<UNNEModelData> ModelData) override;
};