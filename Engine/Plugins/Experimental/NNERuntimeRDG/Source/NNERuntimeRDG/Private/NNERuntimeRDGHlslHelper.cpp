// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGHlslHelper.h"
#include "NNETypes.h"
#include "NNERuntimeRDGTensor.h"
#include "RHI.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	template<typename TTensorInfoParam>
	void FillTensorSizeShaderParameters(const FTensor& Tensor, TTensorInfoParam& OutTensorInfoParam, int32 Idx)
	{
		static_assert(NNE::FTensorShape::MaxRank <= NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS);
		for (int32 i = 0; i < Tensor.GetShape().Rank(); ++i)
		{
			OutTensorInfoParam[i][Idx] = Tensor.GetShape().GetData()[i];
		}
	}
	template void FillTensorSizeShaderParameters(const FTensor&, FTensorInfoParam&, int32);
	template void FillTensorSizeShaderParameters(const FTensor&, FTensorInfoParamArraySpan&, int32);
	
	template<typename TTensorInfoParam>
	void FillTensorStrideShaderParameters(const FTensor& Tensor, TTensorInfoParam& OutTensorInfoParam, int32 Idx, int32 TargetNumdimensionForBroadcast)
	{
		if (TargetNumdimensionForBroadcast == -1)
		{
			TargetNumdimensionForBroadcast = Tensor.GetShape().Rank();
		}
		checkf(TargetNumdimensionForBroadcast >= Tensor.GetShape().Rank(), TEXT("Can't broadcast tensor from rank %d to rank %d, should be inferior or equal."), Tensor.GetShape().Rank(), TargetNumdimensionForBroadcast);
		int32 Offset = TargetNumdimensionForBroadcast - Tensor.GetShape().Rank();

		static_assert(NNE::FTensorShape::MaxRank <= NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS);
		for (int32 i = 7; i >= 0; --i)
		{
			if (i >= TargetNumdimensionForBroadcast || i < Offset)
			{
				OutTensorInfoParam[i][Idx] = 0;
			}
			else if (i == TargetNumdimensionForBroadcast - 1)
			{
				OutTensorInfoParam[i][Idx] = 1;
			}
			else
			{
				OutTensorInfoParam[i][Idx] = OutTensorInfoParam[i + 1][Idx] * Tensor.GetShape().GetData()[i + 1 - Offset];
			}
		}
	}
	template void FillTensorStrideShaderParameters(const FTensor&, FTensorInfoParam&, int32, int32);
	template void FillTensorStrideShaderParameters(const FTensor&, FTensorInfoParamArraySpan&, int32, int32);

	template<typename TTensorInfoParam>
	void FillTensorStrideForBroadcastShaderParameters(const FTensor& Tensor, int32 OutputNumdimension, TTensorInfoParam& OutTensorInfoParam, int32 Idx)
	{
		checkf(OutputNumdimension >= Tensor.GetShape().Rank(), TEXT("Can't broadcast tensor from rank %d to rank %d, should be inferior or equal."), Tensor.GetShape().Rank(), OutputNumdimension);
		FillTensorStrideShaderParameters(Tensor, OutTensorInfoParam, Idx, OutputNumdimension);
		int32 Offset = OutputNumdimension - Tensor.GetShape().Rank();
		for (int32 i = Offset; i < OutputNumdimension; ++i)
		{
			// the stride for broadcast dimension is kept as 0
			if (Tensor.GetShape().GetData()[i - Offset] == 1)
			{
				OutTensorInfoParam[i][Idx] = 0;
			}
		}
	}
	template void FillTensorStrideForBroadcastShaderParameters(const FTensor&, int32, FTensorInfoParam&, int32);
	template void FillTensorStrideForBroadcastShaderParameters(const FTensor&, int32, FTensorInfoParamArraySpan&, int32);

	FIntVector ComputeElementWiseThreadGroups(uint32 ElementCount, uint32 GroupSizeX)
	{
		FIntVector ThreadGroupCount;
		ThreadGroupCount.X = FMath::DivideAndRoundUp(ElementCount, GroupSizeX);
		ThreadGroupCount.Y = 1;
		ThreadGroupCount.Z = 1;
		if (ThreadGroupCount.X > GRHIMaxDispatchThreadGroupsPerDimension.X)
		{
			ThreadGroupCount.Y = FMath::DivideAndRoundUp(ThreadGroupCount.X, GRHIMaxDispatchThreadGroupsPerDimension.X);
			ThreadGroupCount.X = FMath::DivideAndRoundUp(ThreadGroupCount.X, ThreadGroupCount.Y);
			ensure(ThreadGroupCount.Y <= GRHIMaxDispatchThreadGroupsPerDimension.Y);
		}
		return ThreadGroupCount;
	}
} // UE::NNERuntimeRDG::Private::Hlsl