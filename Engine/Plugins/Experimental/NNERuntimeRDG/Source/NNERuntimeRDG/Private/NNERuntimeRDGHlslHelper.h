// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "Misc/AlignedElement.h"
#include "Templates/Function.h"

namespace UE::NNERuntimeRDG::Private { class FTensor; }

#define NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS 8

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	using FTensorInfoParam = TStaticArray<UE::Core::TAlignedElement<FUintVector4, 16>, NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS>;

	struct FTensorInfoParamArraySpan 
	{
		TFunction<FUintVector4& (uint32)> ArrayAtFunction;
		uint32 Offset = 0;
		static constexpr uint32 Length = NXRT_TENSORSTRIDEINFO_MAX_NUM_DIMENSIONS;
		FUintVector4& operator[](uint32 Idx) 
		{
			check(Idx < Length);
			check(ArrayAtFunction);
			return ArrayAtFunction(Offset + Idx); 
		}
	};

	template<typename TTensorInfoParam>
	void FillTensorSizeShaderParameters(const FTensor& Tensor, TTensorInfoParam& OutTensorInfoParam, int32 Idx);
	template<typename TTensorInfoParam>
	void FillTensorStrideShaderParameters(const FTensor& Tensor, TTensorInfoParam& OutTensorInfoParam, int32 Idx, int32 TargetNumdimensionForBroadcast = -1);
	template<typename TTensorInfoParam>
	void FillTensorStrideForBroadcastShaderParameters(const FTensor& Tensor, int32 OutputNumdimension, TTensorInfoParam& OutTensorInfoParam, int32 Idx);
	FIntVector ComputeElementWiseThreadGroups(uint32 ElementCount, uint32 GroupSizeX);
} // namespace UE::NNERuntimeRDG::Private::Hlsl