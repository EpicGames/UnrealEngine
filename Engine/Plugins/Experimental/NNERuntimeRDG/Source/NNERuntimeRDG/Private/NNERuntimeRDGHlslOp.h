// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGBase.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{
	/**
	 * Base class for all Hlsl ML operators
	 */
	struct FOperatorHlsl : public IOperatorRDG, public IPrepareOperator
	{
		virtual ~FOperatorHlsl() = default;
		virtual bool Initialize(TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& Attributes) = 0;
		virtual void Dispatch(FRDGBuilder& GraphBuilder, TConstArrayView<FTensorRDGRef> InInputTensors, TConstArrayView<FTensorRDGRef> InOutputTensors) = 0;
		virtual void OptimizeInputsWeights(TArrayView<FTensorRDGRef> InputWeights) {};
	};

	/**
	  * HLSL ML operator registry
	*/
	typedef TOperatorRegistryRDG<FOperatorHlsl> FOperatorRegistryHlsl;

} // UE::NNERuntimeRDG::Private::Hlsl