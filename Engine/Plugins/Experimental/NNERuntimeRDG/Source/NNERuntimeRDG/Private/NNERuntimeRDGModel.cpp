// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGModel.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataFormat.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"

#include "Serialization/MemoryReader.h"

namespace UE::NNERuntimeRDG::Private
{

bool FModelInstanceRDG::LoadModel(TConstArrayView<uint8> ModelData, FNNERuntimeRDGDataModelFormat& Format, int32 GuidAndVersionSize)
{
	TConstArrayView<uint8> ModelBuffer = { &(ModelData.GetData()[GuidAndVersionSize]), ModelData.Num() - GuidAndVersionSize };

	FMemoryReaderView Reader(ModelBuffer, /*bIsPersitent =*/ true);
	Reader.SetIsPersistent(true);

	Format.Serialize(Reader);

	// Data for base class
	InputSymbolicTensors.Empty();
	OutputSymbolicTensors.Empty();

	// Data for RDG
	AllSymbolicTensorDescs.Empty();
	IntermediateTensorIndices.Empty();
	WeightTensorIndices.Empty();
	InputTensorIndices.Empty();
	OutputTensorIndices.Empty();
	EmptyTensorIndices.Empty();
	OperatorInputTensorIndices.Empty();
	OperatorOutputTensorIndices.Empty();

	// Add tensors
	for (int32 Idx = 0; Idx < Format.Tensors.Num(); ++Idx)
	{
		const FNNERuntimeRDGDataTensorDesc& FormatTensorDesc = Format.Tensors[Idx];

		const NNE::FSymbolicTensorShape SymbolicShape = NNE::FSymbolicTensorShape::Make(FormatTensorDesc.Shape);
		const NNE::FTensorDesc SymbolicTensor = NNE::FTensorDesc::Make(FormatTensorDesc.Name, SymbolicShape, FormatTensorDesc.DataType);

		if (FormatTensorDesc.Type != ENNERuntimeRDGDataTensorType::Empty && FormatTensorDesc.DataType == ENNETensorDataType::None)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor %ls has invalid format: Data type None is reserved for empty tensors.", *SymbolicTensor.GetName());
			return false;
		}

		AllSymbolicTensorDescs.Emplace(Idx, SymbolicTensor);

		if (FormatTensorDesc.Type == ENNERuntimeRDGDataTensorType::Empty)
		{
			EmptyTensorIndices.Emplace(Idx);
		}
		else if (FormatTensorDesc.Type == ENNERuntimeRDGDataTensorType::Input)
		{
			InputTensorIndices.Emplace(Idx);
			InputSymbolicTensors.Emplace(SymbolicTensor);
		}
		else if (FormatTensorDesc.Type == ENNERuntimeRDGDataTensorType::Output)
		{
			OutputTensorIndices.Emplace(Idx);
			OutputSymbolicTensors.Emplace(SymbolicTensor);
		}
		else if (FormatTensorDesc.Type == ENNERuntimeRDGDataTensorType::Intermediate)
		{
			IntermediateTensorIndices.Emplace(Idx);
		}
		else if (FormatTensorDesc.Type == ENNERuntimeRDGDataTensorType::Initializer)
		{
			WeightTensorIndices.Emplace(Idx);
			if (!SymbolicTensor.GetShape().IsConcrete())
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Weight tensor %ls should have a concrete shape", *SymbolicTensor.GetName());
				return false;
			}

			const NNE::FTensorShape TensorShape = NNE::FTensorShape::MakeFromSymbolic(SymbolicTensor.GetShape());
			FTensorRDG& WeightRDG = WeightTensorRDGs.Emplace_GetRef(FTensorRDG::Make(SymbolicTensor, TensorShape, nullptr));

			if (WeightRDG.GetDataSize() != FormatTensorDesc.DataSize)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Weight %ls has incorrect size. Expected %lld bytes, got %lld", *SymbolicTensor.GetName(), FormatTensorDesc.DataSize, WeightRDG.GetDataSize());
				return false;
			}

			const uint8* DataPtr = Format.TensorData.GetData() + FormatTensorDesc.DataOffset;
			TConstArrayView<uint8> DataView = MakeArrayView(DataPtr, FormatTensorDesc.DataSize);

			WeightRDG.SetPreparedData(DataView);
		}
		checkf(FormatTensorDesc.Type != ENNERuntimeRDGDataTensorType::None, TEXT("Unsupported tensor type None"));
	}

	// Loop over all operators in the model and store tensor indices for input/output
	for (int32 Idx = 0; Idx < Format.Operators.Num(); ++Idx)
	{
		OperatorInputTensorIndices.Emplace(Format.Operators[Idx].InTensors);
		OperatorOutputTensorIndices.Emplace(Format.Operators[Idx].OutTensors);
	}

	return true;
}

TConstArrayView<NNE::FTensorDesc> FModelInstanceRDG::GetInputTensorDescs() const
{
	return InputSymbolicTensors;
}

TConstArrayView<NNE::FTensorDesc> FModelInstanceRDG::GetOutputTensorDescs() const
{
	return OutputSymbolicTensors;
}

TConstArrayView<NNE::FTensorShape> FModelInstanceRDG::GetInputTensorShapes() const
{
	return InputTensorShapes;
}

TConstArrayView<NNE::FTensorShape> FModelInstanceRDG::GetOutputTensorShapes() const
{
	return OutputTensorShapes;
}

FModelInstanceRDG::ESetInputTensorShapesStatus FModelInstanceRDG::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	OutputTensorShapes.Reset(OutputTensorIndices.Num());

	//Verify input shape are valid for the model and set InputTensorShapes
	InputTensorShapes.Reset(InInputShapes.Num());
	if (InInputShapes.Num() != InputSymbolicTensors.Num())
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Number of input shapes does not match number of input tensors");
		return ESetInputTensorShapesStatus::Fail;
	}
	for (int32 i = 0; i < InInputShapes.Num(); ++i)
	{
		const NNE::FTensorDesc SymbolicDesc = InputSymbolicTensors[i];
		if (!InInputShapes[i].IsCompatibleWith(SymbolicDesc.GetShape()))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Input shape does not match input tensor %ls of index %d", *SymbolicDesc.GetName(), i);
			return ESetInputTensorShapesStatus::Fail;
		}
	}
	InputTensorShapes = InInputShapes;

	//Allocate and prime all AllTensorRDGRefs with concrete shapes defaulting variables dimension to 1 if needed
	AllTensorRDGRefs.Reserve(AllSymbolicTensorDescs.Num());

	InputTensorRDGs.Reset(InputTensorIndices.Num());
	for (int32 i = 0; i < InputTensorIndices.Num(); ++i)
	{
		const int32 Idx = InputTensorIndices[i];
		const NNE::FTensorDesc& TensorDesc = InputSymbolicTensors[i];
		const NNE::FTensorShape& TensorShape = InputTensorShapes[i];

		InputTensorRDGs.Emplace(FTensorRDG::Make(TensorDesc, TensorShape, nullptr));
		AllTensorRDGRefs.Emplace(Idx, &InputTensorRDGs[i]);
	}

	for (int32 i = 0; i < WeightTensorIndices.Num(); ++i)
	{
		const int32 Idx = WeightTensorIndices[i];

		AllTensorRDGRefs.Emplace(Idx, &WeightTensorRDGs[i]);
	}

	IntermediateTensorRDGs.Reset(IntermediateTensorIndices.Num());
	for (int32 i = 0; i < IntermediateTensorIndices.Num(); ++i)
	{
		const int32 Idx = IntermediateTensorIndices[i];
		const NNE::FTensorDesc& TensorDesc = AllSymbolicTensorDescs[Idx];
		const NNE::FTensorShape TensorShape = NNE::FTensorShape::MakeFromSymbolic(TensorDesc.GetShape());

		IntermediateTensorRDGs.Emplace(FTensorRDG::Make(TensorDesc, TensorShape, nullptr));
		AllTensorRDGRefs.Emplace(Idx, &IntermediateTensorRDGs[i]);
	}

	OutputTensorRDGs.Reset(OutputTensorIndices.Num());
	for (int32 i = 0; i < OutputTensorIndices.Num(); ++i)
	{
		const int32 Idx = OutputTensorIndices[i];
		const NNE::FTensorDesc& TensorDesc = OutputSymbolicTensors[i];
		const NNE::FTensorShape TensorShape = NNE::FTensorShape::MakeFromSymbolic(TensorDesc.GetShape());

		OutputTensorRDGs.Emplace(FTensorRDG::Make(TensorDesc, TensorShape, nullptr));
		AllTensorRDGRefs.Emplace(Idx, &OutputTensorRDGs[i]);
	}

	EmptyTensorRDGs.Reset(EmptyTensorIndices.Num());
	for (int32 i = 0; i < EmptyTensorIndices.Num(); ++i)
	{
		const int32 Idx = EmptyTensorIndices[i];
		const NNE::FTensorDesc& TensorDesc = AllSymbolicTensorDescs[Idx];
		const NNE::FTensorShape TensorShape = NNE::FTensorShape::MakeFromSymbolic(TensorDesc.GetShape());

		EmptyTensorRDGs.Emplace(FTensorRDG::Make(TensorDesc, TensorShape, nullptr));
		AllTensorRDGRefs.Emplace(Idx, &EmptyTensorRDGs[i]);
	}

	checkCode(
		checkf(AllTensorRDGRefs.Num() == AllSymbolicTensorDescs.Num(), TEXT("Some tensor was not allocated for model preparation."));
	);

	//Allow the specific runtime to run shape inference if supported
	if (PrepareTensorShapesAndData() != 0)
	{
		return ESetInputTensorShapesStatus::Fail;
	}

	checkCode(
		for (const TPair<int32, FTensorRDGRef>& Elem : AllTensorRDGRefs)
		{
			checkf(Elem.Value->GetShape().IsCompatibleWith(AllSymbolicTensorDescs[Elem.Key].GetShape()), TEXT("Tensor at index %d have a shape incompatible with model definition."), Elem.Key);
		};
	);

	//Set OutputTensorShapes for the model from preparation result
	for (int32 OutputIndices : OutputTensorIndices)
	{
		OutputTensorShapes.Emplace(AllTensorRDGRefs[OutputIndices]->GetShape());
	}

	check(InputTensorIndices.Num() + OutputTensorIndices.Num() + WeightTensorIndices.Num() + IntermediateTensorIndices.Num() + EmptyTensorIndices.Num() == AllTensorRDGRefs.Num());
	check(InputTensorShapes.Num() == InputSymbolicTensors.Num());
	check(OutputTensorShapes.Num() == OutputSymbolicTensors.Num());
	check(WeightTensorIndices.Num() == WeightTensorRDGs.Num());
	check(AllTensorRDGRefs.Num() == AllSymbolicTensorDescs.Num());
	
	return ESetInputTensorShapesStatus::Ok;
}

FRDGBufferDesc FModelInstanceRDG::CreateRDGBufferDescForTensorRDG(const FTensorRDG& Tensor)
{
	const uint32 ElementByteSize = Tensor.GetElementByteSize();
	const uint32 TotalByteCount = ElementByteSize * Tensor.GetVolume();

	//Round up to next multiple of BUFFER_LENGTH_ALIGNMENT
	const uint32 TargetByteCount = FMath::DivideAndRoundUp(TotalByteCount, (uint32)NNERUNTIMERDGHLSL_BUFFER_LENGTH_ALIGNMENT) * NNERUNTIMERDGHLSL_BUFFER_LENGTH_ALIGNMENT;
	
	const FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(ElementByteSize, TargetByteCount / ElementByteSize);
	return Desc;
}

/**
 * Enqueue operators to RDG, the caller will run the GraphBuilder.Execute()
 */
FModelInstanceRDG::EEnqueueRDGStatus FModelInstanceRDG::EnqueueRDG(FRDGBuilder& RDGBuilder, TConstArrayView<NNE::FTensorBindingRDG> InInputBindings, TConstArrayView<NNE::FTensorBindingRDG> InOutputBindings)
{
	check(IsInRenderingThread());

	int32 Res;

	// Verify the model inputs were prepared
	if (InputTensorShapes.Num() == 0)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Input shapes are not set, please call SetInputTensorShapes.");
		return EEnqueueRDGStatus::Fail;
	}

	Res = SetTensors(RDGBuilder, InputTensorRDGs, InInputBindings);
	if (Res != -1)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Invalid buffer for input tensor binding at index %d", Res);
		return EEnqueueRDGStatus::Fail;
	}

	Res = SetTensors(RDGBuilder, OutputTensorRDGs, InOutputBindings);
	if (Res != -1)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Invalid buffer for output tensor binding at index %d", Res);
		return EEnqueueRDGStatus::Fail;
	}

	
	//Register constant and weights tensors resources to RDG graph, uploading constant tensors if needed
	bool bBuffersUploadedAndRegisteredToRDGGraph = PrepareModelRDG(RDGBuilder);

	//Create temporary buffers for NOT const intermediate tensors
	for (FTensorRDG& TensorRDG : IntermediateTensorRDGs)
	{
		if (!TensorRDG.IsConstant())
		{
			const FRDGBufferDesc BufferDesc = CreateRDGBufferDescForTensorRDG(TensorRDG);
			const FRDGBufferRef TensorBuffer = RDGBuilder.CreateBuffer(BufferDesc, TEXT("NNE.Tensor.Intermediate"), ERDGBufferFlags::None);
			check(!bBuffersUploadedAndRegisteredToRDGGraph || TensorRDG.GetBuffer() == nullptr);
			TensorRDG.SetBuffer(TensorBuffer);
		}
	}

	//Note: DirectML uses RHI buffers instead of RDG buffers
	//For now weights tensors are not uploaded to GPU thus GetBuffer will return nullptr for them.
	if (bBuffersUploadedAndRegisteredToRDGGraph)
	{
		checkCode(
			for (const TPair<int32, FTensorRDGRef>& TensorRDG : AllTensorRDGRefs) 
			{ 
				check(TensorRDG.Value->IsValid()); 
			}
		);
	}

	// We can now dispatch operators
	AddDispatchOps_RenderThread(RDGBuilder);

	return EEnqueueRDGStatus::Ok;
}

int32 FModelInstanceRDG::SetTensors(FRDGBuilder& GraphBuilder, FTensorRDGArray& InTensorRDGs, TConstArrayView<NNE::FTensorBindingRDG> InBindings)
{
	check(InBindings.Num() == InTensorRDGs.Num());
	
	for (int32 Idx = 0; Idx < InBindings.Num(); ++Idx)
	{
		FTensorRDG& TensorRDG = InTensorRDGs[Idx];
		const NNE::FTensorBindingRDG& Binding = InBindings[Idx];
		if (Binding.Buffer == nullptr)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "nullptr buffer encountered.");
			return Idx;
		}
		if (Binding.Buffer->GetSize() % NNERUNTIMERDGHLSL_BUFFER_LENGTH_ALIGNMENT != 0)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Buffer has size %i which is not a multiple of %i", Binding.Buffer->GetSize(), NNERUNTIMERDGHLSL_BUFFER_LENGTH_ALIGNMENT);
			return Idx;
		}
		TensorRDG.SetBuffer(Binding.Buffer);
	}

	return -1;
}

} // namespace UE::NNERuntimeRDG::Private
