// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDG.h"
#include "NNETypes.h"
#include "NNERuntimeRDGBase.h"

namespace UE::NNERuntimeRDG::Private
{
	class FModelInstanceRDG : public NNE::IModelInstanceRDG
	{
	public:
		using ESetInputTensorShapesStatus = NNE::IModelInstanceRDG::ESetInputTensorShapesStatus;
		using EEnqueueRDGStatus = NNE::IModelInstanceRDG::EEnqueueRDGStatus;

		FModelInstanceRDG() {};
		virtual ~FModelInstanceRDG() = default;

		virtual TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;
		virtual TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;
		virtual TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;
		virtual TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;
		virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;
		virtual EEnqueueRDGStatus EnqueueRDG(FRDGBuilder& RDGBuilder, TConstArrayView<NNE::FTensorBindingRDG> InInputBindings, TConstArrayView<NNE::FTensorBindingRDG> InOutputBindings) override;
		
	protected:
		static FRDGBufferDesc CreateRDGBufferDescForTensorRDG(const FTensorRDG& Tensor);
		bool LoadModel(TConstArrayView<uint8> ModelData, FNNERuntimeRDGDataModelFormat& Format, int32 GuidAndVersionSize);
		int32 SetTensors(FRDGBuilder& GraphBuilder, FTensorRDGArray& InTensorRDGs, TConstArrayView<NNE::FTensorBindingRDG> InBindings);

		virtual int PrepareTensorShapesAndData() = 0;
		virtual bool PrepareModelRDG(FRDGBuilder& RDGBuilder) { return false; }
		virtual void AddDispatchOps_RenderThread(FRDGBuilder& GraphBuilder) = 0;

		//Tensor shapes and descriptions
		TArray<NNE::FTensorShape>	InputTensorShapes;
		TArray<NNE::FTensorShape>	OutputTensorShapes;
		TArray<NNE::FTensorDesc>	InputSymbolicTensors;
		TArray<NNE::FTensorDesc>	OutputSymbolicTensors;

		//Tensor descriptor
		TMap<int32, NNE::FTensorDesc>	AllSymbolicTensorDescs;

		//Tensor indices for models
		TArray<int32>				IntermediateTensorIndices;
		TArray<int32>				WeightTensorIndices;
		TArray<int32>				InputTensorIndices;
		TArray<int32>				OutputTensorIndices;
		TArray<int32>				EmptyTensorIndices;

		//Tensor indices by operator
		TArray<TArray<uint32>>		OperatorInputTensorIndices;
		TArray<TArray<uint32>>		OperatorOutputTensorIndices;

		//RDG Tensors
		FTensorRDGRefMap			AllTensorRDGRefs;
		FTensorRDGArray				InputTensorRDGs;
		FTensorRDGArray				OutputTensorRDGs;
		FTensorRDGArray				EmptyTensorRDGs;
		FTensorRDGArray				IntermediateTensorRDGs;
		FTensorRDGArray				WeightTensorRDGs;
	};
	
} // UE::NNERuntimeRDG::Private