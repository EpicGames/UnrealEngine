// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGModelHlsl.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGHlsl.h"
#include "NNERuntimeRDGHlslOp.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

namespace UE::NNERuntimeRDG::Private::Hlsl
{

namespace ModelUtils
{

FOperatorHlsl* OpCreate(const FOperatorDesc& OpDesc, TConstArrayView<NNE::FTensorDesc> InputTensorDescs, TConstArrayView<NNE::FTensorDesc> OutputTensorDescs, const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap)
{
	FOperatorRegistryHlsl::OperatorCreateFunc CreateFn = FOperatorRegistryHlsl::Get()->OpFind(OpDesc);

	if (!CreateFn)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Operator registry failed to find operator: %ls", *OpDesc.GetFullName());
		return nullptr;
	}

	FOperatorHlsl* Op = CreateFn();

	if (!Op->Initialize(InputTensorDescs, OutputTensorDescs, AttributeMap))
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Error initializing operator: %ls", *OpDesc.GetFullName());
		delete Op;
		return nullptr;
	}

	return Op;
}

FTensorRDGRef GetTensorRefIfWeightAndOnlyUsedByOperator(int32 TensorIndex, int32 OperatorIdx,
	TConstArrayView<FNNERuntimeRDGDataOperatorDesc> Operators, TConstArrayView<int32> WeightTensorIndices, FTensorRDGArray& WeightTensorRDGs)
{
	for (int32 Idx = 0; Idx < Operators.Num(); ++Idx)
	{
		if (Idx == OperatorIdx)
			continue;
		for (int32 InputTensorIndex : Operators[Idx].InTensors)
		{
			if (InputTensorIndex == TensorIndex)
				return nullptr;
		}
	}

	int32 WeightIndex;
	if (WeightTensorIndices.Find(TensorIndex, WeightIndex))
	{
		return &WeightTensorRDGs[WeightIndex];
	}

	return nullptr;
}

} // namespace ModelUtils

bool FModelInstance::PrepareModelRDG(FRDGBuilder& RDGBuilder)
{
	//Register constant tensors to graph, uploading if needed
	check(IntermediateTensorRDGs.Num() == ConstantsExternalRDGResources.Num());
	for (int32 Idx = 0; Idx < ConstantsExternalRDGResources.Num(); ++Idx)
	{
		FTensorRDG& Tensor = IntermediateTensorRDGs[Idx];
		TRefCountPtr<FRDGPooledBuffer>& PooledBuffer = ConstantsExternalRDGResources[Idx];

		if (Tensor.HasPreparedData())
		{
			if (!PooledBuffer.IsValid())
			{
				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(Tensor.GetElementByteSize(), Tensor.GetVolume());
				const FRDGBufferRef TransientRDGBuffer = RDGBuilder.CreateBuffer(BufferDesc, TEXT("NNE.Tensor.Intermediate.Constant"), ERDGBufferFlags::None);
				const uint8* TensorData = Tensor.GetPreparedData<uint8>().GetData();
				PooledBuffer = RDGBuilder.ConvertToExternalBuffer(TransientRDGBuffer);

				// Data is copied so model can be released safely or another upload added to the queue
				RDGBuilder.QueueBufferUpload(TransientRDGBuffer, TensorData, Tensor.GetDataSize(), ERDGInitialDataFlags::None);
			}
			check(PooledBuffer.IsValid())
			FRDGBufferRef Buffer = RDGBuilder.RegisterExternalBuffer(PooledBuffer);
			Tensor.SetBuffer(Buffer);
		}
		else
		{
			Tensor.SetBuffer(nullptr);
		}
	}

	//Register weight tensors to graph
	check(WeightTensorRDGs.Num() == WeightsExternalRDGResources.Num());
	for (int32 Idx = 0; Idx < WeightsExternalRDGResources.Num(); ++Idx)
	{
		const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer = WeightsExternalRDGResources[Idx];
		FTensorRDG& Tensor = WeightTensorRDGs[Idx];
		// Only register external buffer when weight tensor is NOT zero-sized.
		if(Tensor.HasPreparedData())
		{
			FRDGBufferRef Buffer = RDGBuilder.RegisterExternalBuffer(PooledBuffer);
			Tensor.SetBuffer(Buffer);
		}
		else
		{
			Tensor.SetBuffer(nullptr);
		}
	}

	return true;
}

FModelInstance::~FModelInstance()
{
	for (FOperatorHlsl* Operator : Operators)
	{
		delete Operator;
	}
	Operators.Empty();
}

bool FModelInstance::Init(TConstArrayView<uint8> ModelData)
{
	check(ModelData.Num() > 0);
	FNNERuntimeRDGDataModelFormat Format;
	int32 GuidSize = sizeof(UNNERuntimeRDGHlslImpl::GUID);
	int32 VersionSize = sizeof(UNNERuntimeRDGHlslImpl::Version);
	
	if (!LoadModel(ModelData, Format, GuidSize+VersionSize))
	{
		return false;
	}

	TArray<NNE::FTensorDesc> Inputs;
	TArray<NNE::FTensorDesc> Outputs;
	TArray<FTensorRDGRef> InputsAsWeights;

	// Loop over all operators in the model and create them
	for (int32 Idx = 0; Idx < Format.Operators.Num(); ++Idx)
	{
		const FOperatorDesc OperatorDesc {{Format.Operators[Idx].TypeName, Format.Operators[Idx].DomainName}, Format.Operators[Idx].Version};
		Inputs.Reset();
		Outputs.Reset();
		InputsAsWeights.Reset();
		UE::NNERuntimeRDGData::Internal::FAttributeMap AttributeMap;

		for (int32 InputTensorIndex : Format.Operators[Idx].InTensors)
		{
			FTensorRDGRef InputAsWeight = ModelUtils::GetTensorRefIfWeightAndOnlyUsedByOperator(InputTensorIndex, Idx, Format.Operators, WeightTensorIndices, WeightTensorRDGs);

			Inputs.Emplace(AllSymbolicTensorDescs[InputTensorIndex]);
			InputsAsWeights.Emplace(InputAsWeight);
		}
		for (int32 OutputTensorIndex : Format.Operators[Idx].OutTensors)
		{
			Outputs.Emplace(AllSymbolicTensorDescs[OutputTensorIndex]);
		}
		for (const FNNERuntimeRDGDataAttributeDesc& Desc : Format.Operators[Idx].Attributes)
		{
			AttributeMap.SetAttribute(Desc.Name, Desc.Value);
		}

		FOperatorHlsl* Op = ModelUtils::OpCreate(OperatorDesc, Inputs, Outputs, AttributeMap);

		if (!Op) //Op.Shader.IsNull())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to create operator:%ls", *OperatorDesc.GetFullName());

			//Note: Need to cleanup operators
			return false;
		}

		Op->OptimizeInputsWeights(InputsAsWeights);

		Operators.Add(Op);
	}

	// Create HLSL tensor and upload to GPU
	PrepareWeights();

	return true;
}

void FModelInstance::AddDispatchOps_RenderThread(FRDGBuilder& GraphBuilder)
{
	static constexpr int32 MaxExpectedInput = 10;
	TArray<FTensorRDGRef, TInlineAllocator<MaxExpectedInput>> InputTensors;

	static constexpr int32 MaxExpectedOutput = 2;
	TArray<FTensorRDGRef, TInlineAllocator<MaxExpectedOutput>> OutputTensors;

	// Add passes for all operators
	for (int32 Idx = 0; Idx < Operators.Num(); ++Idx)
	{
		InputTensors.Reset(OperatorInputTensorIndices.Num());
		for (int32 i : OperatorInputTensorIndices[Idx])
		{
			InputTensors.Add(AllTensorRDGRefs[i]);
		}
		bool AllOutputTensorConstant = true;
		OutputTensors.Reset(OperatorOutputTensorIndices.Num());
		for (int32 i : OperatorOutputTensorIndices[Idx])
		{
			AllOutputTensorConstant &= AllTensorRDGRefs[i]->IsConstant();
			OutputTensors.Add(AllTensorRDGRefs[i]);
		}

		//If all output for operator are constant we don't need to run it.
		if (!AllOutputTensorConstant)
		{
			Operators[Idx]->Dispatch(GraphBuilder, InputTensors, OutputTensors);
		}
	}

	//If a model output is constant (and non-empty) we upload to it (a user provided GPU buffer).
	for (const FTensorRDG& OutputTensor : OutputTensorRDGs)
	{
		if (OutputTensor.HasPreparedData())
		{
			GraphBuilder.QueueBufferUpload(OutputTensor.GetBuffer(), OutputTensor.GetPreparedData<uint8>().GetData(), OutputTensor.GetDataSize(), ERDGInitialDataFlags::None);
		}
	}
}

int FModelInstance::PrepareTensorShapesAndData()
{
	check(AllTensorRDGRefs.Num() == AllSymbolicTensorDescs.Num());
	
	if (Operators.Num() == 0)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "No operators in model");
		return -1;
	}

	// Run model preparation (including shape inference) on all operators
	// This loop could be abstracted to a different runtime/system as it apply on FTensorRef & IPrepareOperator witch are RDG agnostics.
	static constexpr int32 MaxExpectedInput = 10;
	TArray<FTensorRef, TInlineAllocator<MaxExpectedInput>> InputTensors;
	TArray<FTensorRef> OutputTensors;
	TArray<bool> AllInitializedTensors;

	checkCode(
		AllInitializedTensors.Init(false, AllSymbolicTensorDescs.Num());
		for (int32 Idx : InputTensorIndices)
		{
			AllInitializedTensors[Idx] = true;
		}
		for (int32 Idx : WeightTensorIndices)
		{
			AllInitializedTensors[Idx] = true;
		}
		for (int32 Idx : EmptyTensorIndices)
		{
			AllInitializedTensors[Idx] = true;
		}
	);

	//Release uploaded GPU side constants tensors.
	ConstantsExternalRDGResources.Reset();
	ConstantsExternalRDGResources.SetNum(IntermediateTensorRDGs.Num());

	// Run model preparation (including shape inference) on all operators
	// This loop could be abstracted to a different system as it apply on FTensorRef & IPrepareOperator witch are RDG agnostics.
	for (int32 Idx = 0; Idx < Operators.Num(); ++Idx)
	{
		InputTensors.Reset();
		OutputTensors.Reset();

		//Operator inputs
		for (int32 i : OperatorInputTensorIndices[Idx])
		{
			checkf(AllInitializedTensors[i] == true, TEXT("Input tensor %d for operator %d should have been initialized."), i, Idx);
			InputTensors.Emplace(AllTensorRDGRefs[i]);
		}
		//Operator outputs
		for (int32 i : OperatorOutputTensorIndices[Idx])
		{
			OutputTensors.Emplace(AllTensorRDGRefs[i]);
			checkf(AllInitializedTensors[i] == false, TEXT("Output tensor %d for operator %d should not have been initialized yet."), i, Idx);
			checkCode(AllInitializedTensors[i] = true);
		}

		FOperatorHlsl* Op = Operators[Idx];

		if (Op->PrepareOutputs(InputTensors, OutputTensors) != 0)
		{
			//Operator could not prepare the output tensors, meaning we can't allocate
			//output buffer before running the model. This runtime does not support this.
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Could not deduce tensor shapes for this model during shape inference, HLSL runtime wont support the model as it need to precompute all shapes for performance reasons.");
			AllTensorRDGRefs.Reset();
			return -1;
		}
	}

	checkCode(
		for (int i = 0; i < AllInitializedTensors.Num(); ++i)
		{
			checkf(AllInitializedTensors[i], TEXT("Tensor at index %d, was not initialized by model preparation."), i);
		};
	);

	return 0;
}

void FModelInstance::PrepareWeights()
{
	check(WeightsExternalRDGResources.IsEmpty());

	auto EnqueueTensorUpload = [](TArray<TRefCountPtr<FRDGPooledBuffer>>& OutExternalRDGResources,
								  FTensorRDGArray& TensorToUploadRDGs, ERDGInitialDataFlags CopyDataFlag)
	{
		OutExternalRDGResources.Reset();
		OutExternalRDGResources.SetNum(TensorToUploadRDGs.Num());

		bool AtLeastOneConstantNonEmptyTensor = false;
		for (const FTensorRDG& TensorRDG : TensorToUploadRDGs)
		{
			if (TensorRDG.HasPreparedData())
			{
				AtLeastOneConstantNonEmptyTensor = true;
			}
		}
		if (!AtLeastOneConstantNonEmptyTensor)
		{
			return;
		}

		FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(false);

		ENQUEUE_RENDER_COMMAND(FModelInstance_UploadTensors)
		(
			[Signal, &OutExternalRDGResources, &TensorToUploadRDGs, CopyDataFlag](FRHICommandListImmediate& RHICmdList)
			{
				TOptional<ERHIPipeline> Pipeline = RHICmdList.GetPipeline();
				if (Pipeline == ERHIPipeline::None)
				{
					RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
				}

				FRDGBuilder	RDGBuilder(RHICmdList);

				for (int32 i = 0; i < TensorToUploadRDGs.Num(); ++i)
				{
					FTensorRDG& Tensor = TensorToUploadRDGs[i];
					check(!Tensor.HasBuffer());
					if (Tensor.HasPreparedData())
					{

						FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateBufferDesc(Tensor.GetElementByteSize(), Tensor.GetVolume());
						const FRDGBufferRef TransientRDGBuffer = RDGBuilder.CreateBuffer(BufferDesc, TEXT("NNE.Tensor.Weights"), ERDGBufferFlags::None);
						const uint8* TensorData = Tensor.GetPreparedData<uint8>().GetData();

						OutExternalRDGResources[i] = RDGBuilder.ConvertToExternalBuffer(TransientRDGBuffer);
						RDGBuilder.QueueBufferUpload(TransientRDGBuffer, TensorData, Tensor.GetDataSize(), CopyDataFlag);
					}
				}

				RDGBuilder.Execute();

				if (CopyDataFlag == ERDGInitialDataFlags::NoCopy)
				{
					//To prevent any problem if model is released before upload is done to the GPU. To be improved.
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				}


				Signal->Trigger();
			}
		);

		Signal->Wait();	// Wait for render thread to finish

		FGenericPlatformProcess::ReturnSynchEventToPool(Signal);
	};

	// Data is not copied. A GPU sync will happens see EnqueueTensorUpload().
	EnqueueTensorUpload(WeightsExternalRDGResources, WeightTensorRDGs, ERDGInitialDataFlags::NoCopy);
}

TSharedPtr<NNE::IModelInstanceRDG> FModel::CreateModelInstanceRDG()
{
	FModelInstance* ModelInstance = new FModelInstance();

	check(ModelData.IsValid());
	if (!ModelInstance->Init(ModelData->GetView()))
	{
		delete ModelInstance;
		return TSharedPtr<NNE::IModelInstanceRDG>();
	}

	NNE::IModelInstanceRDG* IModelInstance = static_cast<NNE::IModelInstanceRDG*>(ModelInstance);
	return TSharedPtr<NNE::IModelInstanceRDG>(IModelInstance);
}

FModel::FModel(const TSharedPtr<UE::NNE::FSharedModelData>& InModelData) : ModelData(InModelData) {}

} // namespace UE::NNERuntimeRDG::Private::Hlsl
