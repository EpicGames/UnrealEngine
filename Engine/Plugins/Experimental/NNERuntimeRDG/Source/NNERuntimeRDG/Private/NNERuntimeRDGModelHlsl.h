// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGModel.h"
#include "NNEModelData.h"
#include "NNETypes.h"

class FRDGBuilder;

namespace UE::NNERuntimeRDG::Private::Hlsl
{
struct FOperatorHlsl;

class FModelInstance : public FModelInstanceRDG
{
	
public:

	~FModelInstance();

	bool Init(TConstArrayView<uint8> ModelData);

protected:

	virtual int PrepareTensorShapesAndData() override;
	virtual bool PrepareModelRDG(FRDGBuilder& RDGBuilder) override;
	virtual void AddDispatchOps_RenderThread(FRDGBuilder& GraphBuilder) override;

	void PrepareWeights();

private:

	TArray<FOperatorHlsl*>	Operators;
	TArray<TRefCountPtr<FRDGPooledBuffer>> WeightsExternalRDGResources;
	TArray<TRefCountPtr<FRDGPooledBuffer>> ConstantsExternalRDGResources;
};

class FModel : public NNE::IModelRDG
{
public:
	FModel(const TSharedPtr<UE::NNE::FSharedModelData>& InModelData);
	virtual ~FModel() {};

	virtual TSharedPtr<NNE::IModelInstanceRDG> CreateModelInstanceRDG() override;

private:
	TSharedPtr<UE::NNE::FSharedModelData> ModelData;
};

} // namespace UE::NNERuntimeRDG::Private::Hlsl