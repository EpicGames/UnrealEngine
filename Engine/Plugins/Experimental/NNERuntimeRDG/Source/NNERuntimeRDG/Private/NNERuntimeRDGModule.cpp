// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGModule.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/IConsoleManager.h"
#include "NNE.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGHlsl.h"
#include "RHI.h"
#include "UObject/WeakInterfacePtr.h"

static TAutoConsoleVariable<int32> CVarHlslModelOptimization(
	TEXT("nne.hlsl.ModelOptimization"),
	1,
	TEXT("Allows model optimizations when model are cooked for the HLSL runtime.\n")
	TEXT(" 0: disabled\n")
	TEXT(" 1: enabled (default)")
#if !WITH_EDITOR
	, ECVF_ReadOnly
#endif
);

namespace UE::NNERuntimeRDG::Private::Details
{
	bool IsInferenceSupported()
	{
		bool bResult = true;

#ifndef NNE_FORCE_HARDWARE_SUPPORTS_HLSL
		if (GMaxRHIFeatureLevel < ERHIFeatureLevel::SM5)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Minimum feature level required is SM5 for current RHI platform.");
			bResult = false;
		}

		if (!GRHISupportsWaveOperations)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Current RHI platform doesn't support wave operations.");
			bResult = false;
		}

		if (!GRHIGlobals.SupportsNative16BitOps)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Current RHI platform doesn't support native 16-bit operations.");
			bResult = false;
		}
#endif

		return bResult;
	}
}

void FNNERuntimeRDGModule::RegisterRuntime()
{
	NNERuntimeRDGHlsl = NewObject<UNNERuntimeRDGHlslImpl>();
	if (NNERuntimeRDGHlsl.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeInterface(NNERuntimeRDGHlsl.Get());

		NNERuntimeRDGHlsl->Init();
		NNERuntimeRDGHlsl->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeInterface);
	}
}

void FNNERuntimeRDGModule::StartupModule()
{
#ifdef WITH_NNE_RUNTIME_HLSL 
	if (FDataDrivenShaderPlatformInfo::GetSupportsNNEShaders(GMaxRHIShaderPlatform))
	{
		if (UE::NNERuntimeRDG::Private::Details::IsInferenceSupported())
		{
			NNERuntimeRDGHlsl = NewObject<UNNERuntimeRDGHlslImplRDG>();
		}
		else
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Not registering inference for runtime because current hardware is incompatible, consider bypassing by setting the define NNE_FORCE_HARDWARE_SUPPORTS_HLSL.");
		}
	}
	else
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Not registering inference for runtime because current RHI shader platform is not enabled, consider setting the flag bSupportsNNEShaders in DataDrivenPlatformInfo.");
	}

	#if WITH_EDITOR
	if (!NNERuntimeRDGHlsl.IsValid())
	{
		// We can always cook in editor
		NNERuntimeRDGHlsl = NewObject<UNNERuntimeRDGHlslImpl>();
	}
	#endif // WITH_EDITOR

	if (NNERuntimeRDGHlsl.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeInterface(NNERuntimeRDGHlsl.Get());

		NNERuntimeRDGHlsl->Init();
		NNERuntimeRDGHlsl->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeInterface);
	}
#endif // WITH_NNE_RUNTIME_HLSL
}

void FNNERuntimeRDGModule::ShutdownModule()
{

	// NNE runtime RDG Hlsl shutdown
	if (NNERuntimeRDGHlsl.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeInterface(NNERuntimeRDGHlsl.Get());

		UE::NNE::UnregisterRuntime(RuntimeInterface);
		NNERuntimeRDGHlsl->RemoveFromRoot();
		NNERuntimeRDGHlsl.Reset();
	}

}

IMPLEMENT_MODULE(FNNERuntimeRDGModule, NNERuntimeRDG);
