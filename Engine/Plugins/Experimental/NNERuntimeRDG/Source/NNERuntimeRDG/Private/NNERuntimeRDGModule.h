// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UNNERuntimeRDGHlslImpl;

class FNNERuntimeRDGModule : public IModuleInterface
{
private:
	void RegisterRuntime();

public:
	TWeakObjectPtr<UNNERuntimeRDGHlslImpl> NNERuntimeRDGHlsl{ nullptr };

	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};