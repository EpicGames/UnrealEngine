// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "NNETypes.h"

namespace UE::NNERuntimeRDG::Private
{
	/** Concrete tensor with data accessible by graph scheduling */
	class FTensor
	{
	protected:
		FString				Name;
		ENNETensorDataType	DataType;
		NNE::FTensorShape	Shape;
		TArray<uint8>		PreparedData;
		uint64				DataSize = 0;
		uint32				Volume = 0;

		FTensor() = default;

	public:
		const FString& GetName() const
		{
			return Name;
		}

		ENNETensorDataType GetDataType() const
		{
			return DataType;
		}

		uint32 GetElementByteSize() const
		{
			return NNE::GetTensorDataTypeSizeInBytes(DataType);
		}

		const NNE::FTensorShape& GetShape() const
		{
			return Shape;
		}

		template <typename T> TConstArrayView<T> GetPreparedData() const
		{
			const T* DataPtr = reinterpret_cast<const T*>(PreparedData.GetData());
			const int32 ElemSize = sizeof(T);

			check(PreparedData.Num() % ElemSize == 0);
			return MakeArrayView(DataPtr, PreparedData.Num() / ElemSize);
		}

		void SetShape(const NNE::FTensorShape& InShape)
		{
			checkf(!HasPreparedData(), TEXT("Shape cannot be changed once data as been set."));
			check(InShape.Volume() <= TNumericLimits<uint32>::Max());
			Shape = InShape;
			Volume = InShape.Volume();
			DataSize = (uint64)NNE::GetTensorDataTypeSizeInBytes(DataType) * Volume;
		}

		template <typename T> void SetPreparedData(TConstArrayView<T> Data)
		{
			const uint8* DataPtr = reinterpret_cast<const uint8*>(Data.GetData());
			TConstArrayView<uint8> DataAsByte = MakeArrayView(DataPtr, Data.Num() * sizeof(T));
			
			checkf(DataAsByte.Num() == DataSize, TEXT("Incorrect data size, it should match tensor shape and data type."));
			PreparedData.Reset();
			PreparedData.Append(DataAsByte);
		}

		bool IsEmpty() const
		{
			bool IsEmpty = GetDataType() == ENNETensorDataType::None;
			ensureMsgf(!IsEmpty || (Shape.Rank() == 1 && Shape.GetData()[0] == 0), TEXT("Empty tensor should have a shape of [0]."));
			return IsEmpty;
		}

		bool HasPreparedData() const
		{
			return !PreparedData.IsEmpty();
		}

		bool IsConstant() const
		{
			return Volume == 0 || HasPreparedData();
		}

		uint32 GetVolume() const
		{
			return Volume;
		}

		uint64 GetDataSize() const
		{
			return DataSize;
		}

		static FTensor Make(const FString& Name, const NNE::FTensorShape& Shape, ENNETensorDataType DataType)
		{
			FTensor Tensor;
			Tensor.Name = Name;
			Tensor.DataType = DataType;
			Tensor.SetShape(Shape);
			return Tensor;
		}

		static FTensor Make(const NNE::FTensorDesc& TensorDesc, const NNE::FTensorShape& Shape)
		{
			check(Shape.IsCompatibleWith(TensorDesc.GetShape()));
			return Make(TensorDesc.GetName(), Shape, TensorDesc.GetDataType());
		}

		static FTensor MakeFromSymbolicDesc(const NNE::FTensorDesc& TensorDesc)
		{
			return Make(TensorDesc.GetName(), NNE::FTensorShape::MakeFromSymbolic(TensorDesc.GetShape()), TensorDesc.GetDataType());
		}
	};

	using FTensorRef = FTensor*;

} // namespace UE::NNERuntimeRDG::Private