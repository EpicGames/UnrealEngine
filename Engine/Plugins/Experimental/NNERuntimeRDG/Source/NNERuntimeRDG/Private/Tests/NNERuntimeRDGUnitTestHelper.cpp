// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNETypes.h"
#include "Helper/NNERuntimeRDGHelperElementWiseBinary.h"
#include "Helper/NNERuntimeRDGHelperElementWiseUnary.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"

namespace UE::NNERuntimeRDG::Private::Test
{

using namespace NNE;

bool FUnitTestBase::RunTest(const FString& Parameters) { return false; }

FShapeInferenceHelperUnitTestBase::FShapeInferenceHelperUnitTestBase(const FString& InClassName, const FString& InTestName, const FString& InSourceFile, int32 InSourceLine)
	: FUnitTestBase(InClassName), TestName(InTestName), SourceFile(InSourceFile), SourceLine(InSourceLine) {}

FString FShapeInferenceHelperUnitTestBase::GetTestSourceFileName() const { return SourceFile; }

int32 FShapeInferenceHelperUnitTestBase::GetTestSourceFileLine() const { return SourceLine; }
		
FString FShapeInferenceHelperUnitTestBase::GetBeautifiedTestName() const { return TestName; };

void FShapeInferenceHelperUnitTestBase::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	OutBeautifiedNames.Add(TestName);
	OutTestCommands.Add(FString());
}

bool FShapeInferenceHelperUnitTestBase::RunTest(const FString& Parameter) { return false; }

FTensor FShapeInferenceHelperUnitTestBase::MakeTensor(FString Name, TConstArrayView<uint32> Shape, ENNETensorDataType DataType)
{
	FTensorShape TensorShape = FTensorShape::Make(Shape);
	FTensor Tensor = FTensor::Make(Name, TensorShape, DataType);
	return Tensor;
}

FTensor FShapeInferenceHelperUnitTestBase::MakeConstTensor(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<float> Data)
{
	FTensor Tensor = MakeTensor(Name, Shape, ENNETensorDataType::Float);
	check(Data.Num() == Tensor.GetVolume());
	Tensor.SetPreparedData(Data);
	return Tensor;
}

FTensor FShapeInferenceHelperUnitTestBase::MakeConstTensorFloat16(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<FFloat16> Data)
{
	FTensor Tensor = MakeTensor(Name, Shape, ENNETensorDataType::Half);
	check(Data.Num() == Tensor.GetVolume());
	Tensor.SetPreparedData(Data);
	return Tensor;
}

FTensor FShapeInferenceHelperUnitTestBase::MakeConstTensorInt32(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<int32> Data)
{
	FTensor Tensor = MakeTensor(Name, Shape, ENNETensorDataType::Int32);
	check(Data.Num() == Tensor.GetVolume());
	Tensor.SetPreparedData(Data);
	return Tensor;
}

FTensor FShapeInferenceHelperUnitTestBase::MakeConstTensorInt64(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<int64> Data)
{
	FTensor Tensor = MakeTensor(Name, Shape, ENNETensorDataType::Int64);
	check(Data.Num() == Tensor.GetVolume());
	Tensor.SetPreparedData(Data);
	return Tensor;
}

bool FShapeInferenceHelperUnitTestBase::TestBinaryOutputIsOnlyComputedWhenItShould(NNEHlslShaders::Internal::EElementWiseBinaryOperatorType OpType)
{
	FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
	FTensor XC20 = MakeConstTensor(TEXT("XC20"), { 20 }, { 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f });
	FTensor X1 = MakeTensor(TEXT("X"), { 1 });

	//Tests if output tensors are only computed if both inputs are constant and below a certain size
	
	FTensor Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseBinary::Apply(OpType, XC1, XC1, Y);
	UTEST_TRUE(TEXT("Y const if both inputs are const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseBinary::Apply(OpType, X1, XC1, Y);
	UTEST_FALSE(TEXT("Y not const when LHS not const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseBinary::Apply(OpType, XC1, X1, Y);
	UTEST_FALSE(TEXT("Y not const when RHS not const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseBinary::Apply(OpType, X1, X1, Y);
	UTEST_FALSE(TEXT("Y not const when LHS and RHS not const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 20 });
	CPUHelper::ElementWiseBinary::Apply(OpType, XC20, XC1, Y);
	UTEST_FALSE(TEXT("Y not const when LHS too large"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 20 });
	CPUHelper::ElementWiseBinary::Apply(OpType, XC1, XC20, Y);
	UTEST_FALSE(TEXT("Y not const when RHS too large"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 20 });
	CPUHelper::ElementWiseBinary::Apply(OpType, XC20, XC20, Y);
	UTEST_FALSE(TEXT("Y not const when LHS and RHS too large"), Y.HasPreparedData());

	return true;
}

bool FShapeInferenceHelperUnitTestBase::TestUnaryOutputIsOnlyComputedWhenItShould(NNEHlslShaders::Internal::EElementWiseUnaryOperatorType OpType)
{
	FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
	FTensor XC20 = MakeConstTensor(TEXT("XC20"), { 20 }, { 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f });
	FTensor X1 = MakeTensor(TEXT("X"), { 1 });

	//Tests if output tensors are only computed if inputs are constant and below a certain size

	FTensor Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseUnary::Apply(OpType, XC1, 1.0f, 1.0f, 1.0f, Y);
	UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 1 });
	CPUHelper::ElementWiseUnary::Apply(OpType, X1, 1.0f, 1.0f, 1.0f, Y);
	UTEST_FALSE(TEXT("Y not const if input is not const"), Y.HasPreparedData());

	Y = MakeTensor(TEXT("Y"), { 20 });
	CPUHelper::ElementWiseUnary::Apply(OpType, XC20, 1.0f, 1.0f, 1.0f, Y);
	UTEST_FALSE(TEXT("Y not const if input is too large "), Y.HasPreparedData());

	return true;
}

} // namespace UE::NNERuntimeRDG::Private::Test

#endif //WITH_DEV_AUTOMATION_TESTS
