// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGTensor.h"
#include "NNEHlslShadersOperator.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"

namespace UE::NNERuntimeRDG::Private::Test
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST_PRIVATE(FUnitTestBase, FAutomationTestBase, "System.Engine.MachineLearning.NNE.RDG.UnitTest.Base", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags_FeatureMask | EAutomationTestFlags::EngineFilter, __FILE__, __LINE__)

	class FShapeInferenceHelperUnitTestBase : public FUnitTestBase
	{
		//Automation tests
		FString TestName;
		FString SourceFile;
		int32 SourceLine;
		
	public:
		FShapeInferenceHelperUnitTestBase(const FString& InClassName, const FString& InTestName, const FString& InSourceFile, int32 InSourceLine);
		virtual ~FShapeInferenceHelperUnitTestBase() {}
		virtual FString GetTestSourceFileName() const override;
		virtual int32 GetTestSourceFileLine() const override;
		
	protected:
		virtual FString GetBeautifiedTestName() const override;
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override;
		virtual bool RunTest(const FString& Parameter) override;

		//Shape inference unit test helpers
		static FTensor MakeTensor(FString Name, TConstArrayView<uint32> Shape, ENNETensorDataType DataType = ENNETensorDataType::Float);
		static FTensor MakeConstTensor(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<float> Data);
		static FTensor MakeConstTensorFloat16(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<FFloat16> Data);
		static FTensor MakeConstTensorInt32(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<int32> Data);
		static FTensor MakeConstTensorInt64(FString Name, TConstArrayView<uint32> Shape, TConstArrayView<int64> Data);
		bool TestUnaryOutputIsOnlyComputedWhenItShould(NNEHlslShaders::Internal::EElementWiseUnaryOperatorType OpType);
		bool TestBinaryOutputIsOnlyComputedWhenItShould(NNEHlslShaders::Internal::EElementWiseBinaryOperatorType OpType);
	};

	#define IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST( TClass, PrettyName ) \
	class TClass : public FShapeInferenceHelperUnitTestBase \
	{ \
	public: \
		TClass(const FString& InClassName, const FString& InTestName, const FString& InSourceFile, int32 InSourceLine) \
			: FShapeInferenceHelperUnitTestBase(InClassName, InTestName, InSourceFile, InSourceLine) {} \
		virtual ~TClass() {} \
    \
	protected: \
		virtual bool RunTest(const FString& Parameter) override; \
	}; \
	namespace\
	{\
		TClass TClass##AutomationTestInstance( TEXT(#TClass), PrettyName, __FILE__, __LINE__ );\
	}



} // namespace UE::NNERuntimeRDG::Private::Test

#endif //WITH_DEV_AUTOMATION_TESTS
