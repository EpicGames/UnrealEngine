// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Helper/NNERuntimeRDGHelperElementWiseBinary.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNETypes.h"


#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::BinaryOp
{

#if WITH_DEV_AUTOMATION_TESTS
	
	using namespace NNE;

	using EElementWiseBinaryOperatorType = NNEHlslShaders::Internal::EElementWiseBinaryOperatorType;

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperAdd, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Add");
	bool FElementWiseBinaryCPUHelperAdd::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Add))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Add, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1+XC1"), Y.GetPreparedData<float>()[0], 1.0f + 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Add, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2+XC1[0]"), Y.GetPreparedData<float>()[0], 1.0f + 1.0f);
		UTEST_EQUAL(TEXT("XC1x2+XC1[1]"), Y.GetPreparedData<float>()[1], 2.0f + 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Add, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2+XC1x2[0]"), Y.GetPreparedData<float>()[0], 1.0f + 1.0f);
		UTEST_EQUAL(TEXT("XC1x2+XC1x2[1]"), Y.GetPreparedData<float>()[1], 2.0f + 2.0f);

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Add, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2+XC2x1[0]"), Y.GetPreparedData<float>()[0], 1.0f + 3.0f);
		UTEST_EQUAL(TEXT("XC1x2+XC2x1[1]"), Y.GetPreparedData<float>()[1], 2.0f + 3.0f);
		UTEST_EQUAL(TEXT("XC1x2+XC2x1[2]"), Y.GetPreparedData<float>()[2], 1.0f + 4.0f);
		UTEST_EQUAL(TEXT("XC1x2+XC2x1[3]"), Y.GetPreparedData<float>()[3], 2.0f + 4.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperDiv, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Div");
	bool FElementWiseBinaryCPUHelperDiv::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Div))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Div, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1/XC1"), Y.GetPreparedData<float>()[0], 1.0f / 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Div, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2/XC1[0]"), Y.GetPreparedData<float>()[0], 1.0f / 1.0f);
		UTEST_EQUAL(TEXT("XC1x2/XC1[1]"), Y.GetPreparedData<float>()[1], 2.0f / 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Div, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2/XC1x2[0]"), Y.GetPreparedData<float>()[0], 1.0f / 1.0f);
		UTEST_EQUAL(TEXT("XC1x2/XC1x2[1]"), Y.GetPreparedData<float>()[1], 2.0f / 2.0f);

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Div, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2/XC2x1[0]"), Y.GetPreparedData<float>()[0], 1.0f / 3.0f);
		UTEST_EQUAL(TEXT("XC1x2/XC2x1[1]"), Y.GetPreparedData<float>()[1], 2.0f / 3.0f);
		UTEST_EQUAL(TEXT("XC1x2/XC2x1[2]"), Y.GetPreparedData<float>()[2], 1.0f / 4.0f);
		UTEST_EQUAL(TEXT("XC1x2/XC2x1[3]"), Y.GetPreparedData<float>()[3], 2.0f / 4.0f);

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperMul, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Mul");
	bool FElementWiseBinaryCPUHelperMul::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Mul))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1*XC1"), Y.GetPreparedData<float>()[0], 1.0f * 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1[0]"), Y.GetPreparedData<float>()[0], 1.0f * 1.0f);
		UTEST_EQUAL(TEXT("XC1x2*XC1[1]"), Y.GetPreparedData<float>()[1], 2.0f * 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[0]"), Y.GetPreparedData<float>()[0], 1.0f * 1.0f);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[1]"), Y.GetPreparedData<float>()[1], 2.0f * 2.0f);

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[0]"), Y.GetPreparedData<float>()[0], 1.0f * 3.0f);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[1]"), Y.GetPreparedData<float>()[1], 2.0f * 3.0f);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[2]"), Y.GetPreparedData<float>()[2], 1.0f * 4.0f);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[3]"), Y.GetPreparedData<float>()[3], 2.0f * 4.0f);

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperMulInt32, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.MulInt32");
	bool FElementWiseBinaryCPUHelperMulInt32::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensorInt32(TEXT("XC1"), { 1 }, { 1 });
		FTensor XC1x2 = MakeConstTensorInt32(TEXT("XC1x2"), { 1,2 }, { 1, 2 });
		FTensor XC2x1 = MakeConstTensorInt32(TEXT("XC2x1"), { 2,1 }, { 3, 4 });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Mul))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 }, ENNETensorDataType::Int32);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1*XC1"), Y.GetPreparedData<int32>()[0], 1 * 1);

		Y = MakeTensor(TEXT("Y"), { 1,2 }, ENNETensorDataType::Int32);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1[0]"), Y.GetPreparedData<int32>()[0], 1 * 1);
		UTEST_EQUAL(TEXT("XC1x2*XC1[1]"), Y.GetPreparedData<int32>()[1], 2 * 1);

		Y = MakeTensor(TEXT("Y"), { 1,2 }, ENNETensorDataType::Int32);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[0]"), Y.GetPreparedData<int32>()[0], 1 * 1);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[1]"), Y.GetPreparedData<int32>()[1], 2 * 2);

		Y = MakeTensor(TEXT("Y"), { 2,2 }, ENNETensorDataType::Int32);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[0]"), Y.GetPreparedData<int32>()[0], 1 * 3);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[1]"), Y.GetPreparedData<int32>()[1], 2 * 3);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[2]"), Y.GetPreparedData<int32>()[2], 1 * 4);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[3]"), Y.GetPreparedData<int32>()[3], 2 * 4);

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperMulInt64, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.MulInt64");
	bool FElementWiseBinaryCPUHelperMulInt64::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensorInt64(TEXT("XC1"), { 1 }, { 1 });
		FTensor XC1x2 = MakeConstTensorInt64(TEXT("XC1x2"), { 1,2 }, { 1, 2 });
		FTensor XC2x1 = MakeConstTensorInt64(TEXT("XC2x1"), { 2,1 }, { 3, 4 });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Mul))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 }, ENNETensorDataType::Int64);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1*XC1"), Y.GetPreparedData<int64>()[0], (int64)1 * 1);

		Y = MakeTensor(TEXT("Y"), { 1,2 }, ENNETensorDataType::Int64);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1[0]"), Y.GetPreparedData<int64>()[0], (int64)1 * 1);
		UTEST_EQUAL(TEXT("XC1x2*XC1[1]"), Y.GetPreparedData<int64>()[1], (int64)2 * 1);

		Y = MakeTensor(TEXT("Y"), { 1,2 }, ENNETensorDataType::Int64);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[0]"), Y.GetPreparedData<int64>()[0], (int64)1 * 1);
		UTEST_EQUAL(TEXT("XC1x2*XC1x2[1]"), Y.GetPreparedData<int64>()[1], (int64)2 * 2);

		Y = MakeTensor(TEXT("Y"), { 2,2 }, ENNETensorDataType::Int64);
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mul, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[0]"), Y.GetPreparedData<int64>()[0], (int64)1 * 3);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[1]"), Y.GetPreparedData<int64>()[1], (int64)2 * 3);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[2]"), Y.GetPreparedData<int64>()[2], (int64)1 * 4);
		UTEST_EQUAL(TEXT("XC1x2*XC2x1[3]"), Y.GetPreparedData<int64>()[3], (int64)2 * 4);

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperSub, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Sub");
	bool FElementWiseBinaryCPUHelperSub::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Sub))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Sub, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1-XC1"), Y.GetPreparedData<float>()[0], 1.0f - 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Sub, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2-XC1[0]"), Y.GetPreparedData<float>()[0], 1.0f - 1.0f);
		UTEST_EQUAL(TEXT("XC1x2-XC1[1]"), Y.GetPreparedData<float>()[1], 2.0f - 1.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Sub, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2-XC1x2[0]"), Y.GetPreparedData<float>()[0], 1.0f - 1.0f);
		UTEST_EQUAL(TEXT("XC1x2-XC1x2[1]"), Y.GetPreparedData<float>()[1], 2.0f - 2.0f);

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Sub, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2-XC2x1[0]"), Y.GetPreparedData<float>()[0], 1.0f - 3.0f);
		UTEST_EQUAL(TEXT("XC1x2-XC2x1[1]"), Y.GetPreparedData<float>()[1], 2.0f - 3.0f);
		UTEST_EQUAL(TEXT("XC1x2-XC2x1[2]"), Y.GetPreparedData<float>()[2], 1.0f - 4.0f);
		UTEST_EQUAL(TEXT("XC1x2-XC2x1[3]"), Y.GetPreparedData<float>()[3], 2.0f - 4.0f);

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperMod, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Mod");
	bool FElementWiseBinaryCPUHelperMod::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Mod))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mod, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1%XC1"), Y.GetPreparedData<float>()[0], FMath::Fmod(1.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mod, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2%XC1[0]"), Y.GetPreparedData<float>()[0], FMath::Fmod(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2%XC1[1]"), Y.GetPreparedData<float>()[1], FMath::Fmod(2.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mod, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2%XC1x2[0]"), Y.GetPreparedData<float>()[0], FMath::Fmod(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2%XC1x2[1]"), Y.GetPreparedData<float>()[1], FMath::Fmod(2.0f, 2.0f));

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Mod, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2%XC2x1[0]"), Y.GetPreparedData<float>()[0], FMath::Fmod(1.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2%XC2x1[1]"), Y.GetPreparedData<float>()[1], FMath::Fmod(2.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2%XC2x1[2]"), Y.GetPreparedData<float>()[2], FMath::Fmod(1.0f, 4.0f));
		UTEST_EQUAL(TEXT("XC1x2%XC2x1[3]"), Y.GetPreparedData<float>()[3], FMath::Fmod(2.0f, 4.0f));

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperPow, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Pow");
	bool FElementWiseBinaryCPUHelperPow::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Pow))
		{
			return false;
		}

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Pow, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1^XC1"), Y.GetPreparedData<float>()[0], FMath::Pow(1.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Pow, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2^XC1[0]"), Y.GetPreparedData<float>()[0], FMath::Pow(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2^XC1[1]"), Y.GetPreparedData<float>()[1], FMath::Pow(2.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Pow, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2^XC1x2[0]"), Y.GetPreparedData<float>()[0], FMath::Pow(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2^XC1x2[1]"), Y.GetPreparedData<float>()[1], FMath::Pow(2.0f, 2.0f));

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Pow, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2^XC2x1[0]"), Y.GetPreparedData<float>()[0], FMath::Pow(1.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2^XC2x1[1]"), Y.GetPreparedData<float>()[1], FMath::Pow(2.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2^XC2x1[2]"), Y.GetPreparedData<float>()[2], FMath::Pow(1.0f, 4.0f));
		UTEST_EQUAL(TEXT("XC1x2^XC2x1[3]"), Y.GetPreparedData<float>()[3], FMath::Pow(2.0f, 4.0f));

		return true;
	};

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseBinaryCPUHelperPrelu, "System.Engine.MachineLearning.NNE.RDG.UnitTest.BinaryHelper.Prelu");
	bool FElementWiseBinaryCPUHelperPrelu::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 3.0f, 4.0f });

		if (!TestBinaryOutputIsOnlyComputedWhenItShould(EElementWiseBinaryOperatorType::Prelu))
		{
			return false;
		}

		auto Prelu = [](float X, float Y) { return (X < 0.0f) ? (Y * X) : X; };

		//Test output tensor math is correct when both inputs are constant including broadcasting
		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Prelu, XC1, XC1, Y);
		UTEST_EQUAL(TEXT("XC1_XC1"), Y.GetPreparedData<float>()[0], Prelu(1.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Prelu, XC1x2, XC1, Y);
		UTEST_EQUAL(TEXT("XC1x2_XC1[0]"), Y.GetPreparedData<float>()[0], Prelu(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2_XC1[1]"), Y.GetPreparedData<float>()[1], Prelu(2.0f, 1.0f));

		Y = MakeTensor(TEXT("Y"), { 1,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Prelu, XC1x2, XC1x2, Y);
		UTEST_EQUAL(TEXT("XC1x2_XC1x2[0]"), Y.GetPreparedData<float>()[0], Prelu(1.0f, 1.0f));
		UTEST_EQUAL(TEXT("XC1x2_XC1x2[1]"), Y.GetPreparedData<float>()[1], Prelu(2.0f, 2.0f));

		Y = MakeTensor(TEXT("Y"), { 2,2 });
		CPUHelper::ElementWiseBinary::Apply(EElementWiseBinaryOperatorType::Prelu, XC1x2, XC2x1, Y);
		UTEST_EQUAL(TEXT("XC1x2_XC2x1[0]"), Y.GetPreparedData<float>()[0], Prelu(1.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2_XC2x1[1]"), Y.GetPreparedData<float>()[1], Prelu(2.0f, 3.0f));
		UTEST_EQUAL(TEXT("XC1x2_XC2x1[2]"), Y.GetPreparedData<float>()[2], Prelu(1.0f, 4.0f));
		UTEST_EQUAL(TEXT("XC1x2_XC2x1[3]"), Y.GetPreparedData<float>()[3], Prelu(2.0f, 4.0f));

		return true;
	};

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::BinaryOp
