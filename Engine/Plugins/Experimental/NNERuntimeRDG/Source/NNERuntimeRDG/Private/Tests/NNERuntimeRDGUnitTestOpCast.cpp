// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Helper/NNERuntimeRDGHelperCast.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNETypes.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::CastOp
{

#if WITH_DEV_AUTOMATION_TESTS

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperConstOuput, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.ConstOutput");
	bool FCastCPUHelperConstOuput::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC20 = MakeConstTensor(TEXT("XC20"), { 20 }, { 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f });
		FTensor X1 = MakeTensor(TEXT("X"), { 1 });

		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::Cast::Apply(XC1 , Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::Cast::Apply(X1, Y);
		UTEST_FALSE(TEXT("Y not const if input not const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 20 });
		CPUHelper::Cast::Apply(XC20, Y);
		UTEST_FALSE(TEXT("Y not const if input is too large "), Y.HasPreparedData());

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperRank1FromFloat, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.Rank1FromFloat");
	bool FCastCPUHelperRank1FromFloat::RunTest(const FString& Parameter)
	{
		FTensor XC2 = MakeConstTensor(TEXT("XC2"), { 2 }, { -1.0f, 1.5f });

		FTensor Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float->float)[0]"), Y.GetPreparedData<float>()[0], -1.0f);
		UTEST_EQUAL(TEXT("Cast(float->float)[1]"), Y.GetPreparedData<float>()[1], 1.5f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Half);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float->half)[0]"), Y.GetPreparedData<FFloat16>()[0], -1.0f);
		UTEST_EQUAL(TEXT("Cast(float->half)[1]"), Y.GetPreparedData<FFloat16>()[1], 1.5f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float->int32)[0]"), Y.GetPreparedData<int32>()[0], (int32)-1);
		UTEST_EQUAL(TEXT("Cast(float->int32)[1]"), Y.GetPreparedData<int32>()[1], (int32)1);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float->int64)[0]"), Y.GetPreparedData<int64>()[0], (int64)-1);
		UTEST_EQUAL(TEXT("Cast(float->int64)[1]"), Y.GetPreparedData<int64>()[1], (int64)1);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperRank1FromFloat16, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.Rank1FromFloat16");
	bool FCastCPUHelperRank1FromFloat16::RunTest(const FString& Parameter)
	{
		FTensor XC2 = MakeConstTensorFloat16(TEXT("XC2"), { 2 }, { -1.0f, 1.5f });

		FTensor Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(half->float)[0]"), Y.GetPreparedData<float>()[0], -1.0f);
		UTEST_EQUAL(TEXT("Cast(half->float)[1]"), Y.GetPreparedData<float>()[1], 1.5f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Half);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(half->half)[0]"), Y.GetPreparedData<FFloat16>()[0], -1.0f);
		UTEST_EQUAL(TEXT("Cast(half->half)[1]"), Y.GetPreparedData<FFloat16>()[1], 1.5f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(half->int32)[0]"), Y.GetPreparedData<int32>()[0], (int32)-1);
		UTEST_EQUAL(TEXT("Cast(half->int32)[1]"), Y.GetPreparedData<int32>()[1], (int32)1);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(half->int64)[0]"), Y.GetPreparedData<int64>()[0], (int64)-1);
		UTEST_EQUAL(TEXT("Cast(half->int64)[1]"), Y.GetPreparedData<int64>()[1], (int64)1);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperRank1FromInt32, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.Rank1FromInt32");
	bool FCastCPUHelperRank1FromInt32::RunTest(const FString& Parameter)
	{
		FTensor XC2 = MakeConstTensorInt32(TEXT("XC2"), { 2 }, { -2, 0 });

		FTensor Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int32->float)[0]"), Y.GetPreparedData<float>()[0], -2.0f);
		UTEST_EQUAL(TEXT("Cast(int32->float)[1]"), Y.GetPreparedData<float>()[1], 0.0f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Half);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int32->half)[0]"), Y.GetPreparedData<FFloat16>()[0], -2.0f);
		UTEST_EQUAL(TEXT("Cast(int32->half)[1]"), Y.GetPreparedData<FFloat16>()[1], 0.0f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int32->int32)[0]"), Y.GetPreparedData<int32>()[0], (int32)-2);
		UTEST_EQUAL(TEXT("Cast(int32->int32)[1]"), Y.GetPreparedData<int32>()[1], (int32)0);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int32->int64)[0]"), Y.GetPreparedData<int64>()[0], (int64)-2);
		UTEST_EQUAL(TEXT("Cast(int32->int64)[1]"), Y.GetPreparedData<int64>()[1], (int64)0);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperRank1FromInt64, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.Rank1FromInt64");
	bool FCastCPUHelperRank1FromInt64::RunTest(const FString& Parameter)
	{
		FTensor XC2 = MakeConstTensorInt64(TEXT("XC2"), { 2 }, { (int64)-4, (int64)10});

		FTensor Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int64->float)[0]"), Y.GetPreparedData<float>()[0], -4.0f);
		UTEST_EQUAL(TEXT("Cast(int64->float)[1]"), Y.GetPreparedData<float>()[1], 10.0f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Half);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int64->half)[0]"), Y.GetPreparedData<FFloat16>()[0], -4.0f);
		UTEST_EQUAL(TEXT("Cast(int64->half)[1]"), Y.GetPreparedData<FFloat16>()[1], 10.0f);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int64->int32)[0]"), Y.GetPreparedData<int32>()[0], (int32)-4);
		UTEST_EQUAL(TEXT("Cast(int64->int32)[1]"), Y.GetPreparedData<int32>()[1], (int32)10);

		Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
		CPUHelper::Cast::Apply(XC2, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(int64->int64)[0]"), Y.GetPreparedData<int64>()[0], (int64)-4);
		UTEST_EQUAL(TEXT("Cast(int64->int64)[1]"), Y.GetPreparedData<int64>()[1], (int64)10);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FCastCPUHelperRank3, "System.Engine.MachineLearning.NNE.RDG.UnitTest.CastHelper.Rank3");
	bool FCastCPUHelperRank3::RunTest(const FString& Parameter)
	{
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { -1.0f, -0.5f, 0.0f, 0.1f, 0.6f, 1.5f });

		FTensor Y = MakeTensor(TEXT("Y"), { 1,2,3 }, ENNETensorDataType::Float);
		CPUHelper::Cast::Apply(XC1x2x3, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float,float)[0]"), Y.GetPreparedData<float>()[0], -1.0f);
		UTEST_EQUAL(TEXT("Cast(float,float)[1]"), Y.GetPreparedData<float>()[1], -0.5f);
		UTEST_EQUAL(TEXT("Cast(float,float)[2]"), Y.GetPreparedData<float>()[2],  0.0f);
		UTEST_EQUAL(TEXT("Cast(float,float)[3]"), Y.GetPreparedData<float>()[3],  0.1f);
		UTEST_EQUAL(TEXT("Cast(float,float)[4]"), Y.GetPreparedData<float>()[4],  0.6f);
		UTEST_EQUAL(TEXT("Cast(float,float)[5]"), Y.GetPreparedData<float>()[5],  1.5f);

		Y = MakeTensor(TEXT("Y"), { 1,2,3 }, ENNETensorDataType::Int32);
		CPUHelper::Cast::Apply(XC1x2x3, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float,int32)[0]"), Y.GetPreparedData<int32>()[0], -1);
		UTEST_EQUAL(TEXT("Cast(float,int32)[1]"), Y.GetPreparedData<int32>()[1], 0);
		UTEST_EQUAL(TEXT("Cast(float,int32)[2]"), Y.GetPreparedData<int32>()[2], 0);
		UTEST_EQUAL(TEXT("Cast(float,int32)[3]"), Y.GetPreparedData<int32>()[3], 0);
		UTEST_EQUAL(TEXT("Cast(float,int32)[4]"), Y.GetPreparedData<int32>()[4], 0);
		UTEST_EQUAL(TEXT("Cast(float,int32)[5]"), Y.GetPreparedData<int32>()[5], 1);

		Y = MakeTensor(TEXT("Y"), { 1,2,3 }, ENNETensorDataType::Int64);
		CPUHelper::Cast::Apply(XC1x2x3, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cast(float,int64)[0]"), Y.GetPreparedData<int64>()[0], (int64)-1);
		UTEST_EQUAL(TEXT("Cast(float,int64)[1]"), Y.GetPreparedData<int64>()[1], (int64)0);
		UTEST_EQUAL(TEXT("Cast(float,int64)[2]"), Y.GetPreparedData<int64>()[2], (int64)0);
		UTEST_EQUAL(TEXT("Cast(float,int64)[3]"), Y.GetPreparedData<int64>()[3], (int64)0);
		UTEST_EQUAL(TEXT("Cast(float,int64)[4]"), Y.GetPreparedData<int64>()[4], (int64)0);
		UTEST_EQUAL(TEXT("Cast(float,int64)[5]"), Y.GetPreparedData<int64>()[5], (int64)1);

		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::CastOp
