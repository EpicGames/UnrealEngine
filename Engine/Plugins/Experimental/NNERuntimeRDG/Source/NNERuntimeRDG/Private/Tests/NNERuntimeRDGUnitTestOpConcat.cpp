// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Helper/NNERuntimeRDGHelperConcat.h"
#include "NNERuntimeRDGTensor.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNETypes.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::ConcatOp
{

#if WITH_DEV_AUTOMATION_TESTS

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FConcatCPUHelperConstOuput, "System.Engine.MachineLearning.NNE.RDG.UnitTest.ConcatHelper.ConstOutput");
	bool FConcatCPUHelperConstOuput::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC20 = MakeConstTensor(TEXT("XC20"), { 20 }, { 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f });
		FTensor X1 = MakeTensor(TEXT("X"), { 1 });

		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::Concat::Apply({ &XC1 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 2 });
		CPUHelper::Concat::Apply({ &XC1, &XC1 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 2 });
		CPUHelper::Concat::Apply({ &X1, &XC1 }, Y, 0);
		UTEST_FALSE(TEXT("Y not const if at least on input not const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 20 });
		CPUHelper::Concat::Apply({ &XC20 }, Y, 0);
		UTEST_FALSE(TEXT("Y not const if input is too large "), Y.HasPreparedData());

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FConcatCPUHelperRank1, "System.Engine.MachineLearning.NNE.RDG.UnitTest.ConcatHelper.Rank1");
	bool FConcatCPUHelperRank1::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC1v2 = MakeConstTensor(TEXT("XC1"), { 1 }, { 2.0f });

		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CPUHelper::Concat::Apply({ &XC1 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1,XC1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);

		Y = MakeTensor(TEXT("Y"), { 2 });
		CPUHelper::Concat::Apply({ &XC1, &XC1v2 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1,XC1v2)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1,XC1v2)[1]"), Y.GetPreparedData<float>()[1], 2.0f);

		Y = MakeTensor(TEXT("Y"), { 3 });
		CPUHelper::Concat::Apply({ &XC1, &XC1v2, &XC1 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1,XC1v2,XC1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1,XC1v2,XC1)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Concat(XC1,XC1v2,XC1)[2]"), Y.GetPreparedData<float>()[2], 1.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FConcatCPUHelperRank3, "System.Engine.MachineLearning.NNE.RDG.UnitTest.ConcatHelper.Rank3");
	bool FConcatCPUHelperRank3::RunTest(const FString& Parameter)
	{
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });
		FTensor XC1x2x1 = MakeConstTensor(TEXT("XC1x2x1"), { 1,2,1 }, { -1.0f, -2.0f });
		FTensor XC1x1x3 = MakeConstTensor(TEXT("XC1x1x3"), { 1,1,3 }, { 10.0f, 11.0f, 12.0f });

		FTensor Y = MakeTensor(TEXT("Y"), { 1,2,4 });
		CPUHelper::Concat::Apply({ &XC1x2x3, &XC1x2x1 }, Y, 2);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[2]"), Y.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[3]"), Y.GetPreparedData<float>()[3], -1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[4]"), Y.GetPreparedData<float>()[4], 4.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[5]"), Y.GetPreparedData<float>()[5], 5.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[6]"), Y.GetPreparedData<float>()[6], 6.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x1)[7]"), Y.GetPreparedData<float>()[7], -2.0f);

		Y = MakeTensor(TEXT("Y"), { 1,3,3 });
		CPUHelper::Concat::Apply({ &XC1x2x3, &XC1x1x3 }, Y, 1);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[2]"), Y.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[3]"), Y.GetPreparedData<float>()[3], 4.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[4]"), Y.GetPreparedData<float>()[4], 5.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[5]"), Y.GetPreparedData<float>()[5], 6.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[6]"), Y.GetPreparedData<float>()[6], 10.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[7]"), Y.GetPreparedData<float>()[7], 11.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x1x3)[8]"), Y.GetPreparedData<float>()[8], 12.0f);

		Y = MakeTensor(TEXT("Y"), { 2,2,3 });
		CPUHelper::Concat::Apply({ &XC1x2x3, &XC1x2x3 }, Y, 0);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[2]"), Y.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[3]"), Y.GetPreparedData<float>()[3], 4.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[4]"), Y.GetPreparedData<float>()[4], 5.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[5]"), Y.GetPreparedData<float>()[5], 6.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[6]"), Y.GetPreparedData<float>()[6], 1.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[7]"), Y.GetPreparedData<float>()[7], 2.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[8]"), Y.GetPreparedData<float>()[8], 3.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[9]"), Y.GetPreparedData<float>()[9], 4.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[10]"), Y.GetPreparedData<float>()[10], 5.0f);
		UTEST_EQUAL(TEXT("Concat(XC1x2x3,XC1x2x3)[11]"), Y.GetPreparedData<float>()[11], 6.0f);

		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::ConcatOp
