// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "Helper/NNERuntimeRDGHelperGather.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::GatherOp
{

#if WITH_DEV_AUTOMATION_TESTS

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FGatherCPUHelperConstOuput, "System.Engine.MachineLearning.NNE.RDG.UnitTest.GatherHelper.ConstOutput");
	bool FGatherCPUHelperConstOuput::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensorInt64(TEXT("XC1"), { 1 }, { 0 });
		FTensor XC20 = MakeConstTensorInt64(TEXT("XC20"), { 20 }, { 0,1,2,3,4,5,6,7,8,9,0,1,2,3,4,5,6,7,8,9 });
		FTensor X1 = MakeTensor(TEXT("X"), { 0 }, ENNETensorDataType::Int64);

		FTensor Y = MakeTensor(TEXT("Y"), { 1 }, ENNETensorDataType::Int64);
		CPUHelper::Gather::Apply(XC1 , XC1, 0, Y);
		UTEST_TRUE(TEXT("Y const if inputs are const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 1 }, ENNETensorDataType::Int64);
		CPUHelper::Gather::Apply(XC1, X1, 0, Y);
		UTEST_FALSE(TEXT("Y not const if data not const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 1 }, ENNETensorDataType::Int64);
		CPUHelper::Gather::Apply(X1, XC1, 0, Y);
		UTEST_FALSE(TEXT("Y not const if indices not const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 20 }, ENNETensorDataType::Int64);
		CPUHelper::Gather::Apply(XC20, XC1, 0, Y);
		UTEST_FALSE(TEXT("Y not const if input is too large "), Y.HasPreparedData());

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FGatherCPUHelperRank1Int64Indices, "System.Engine.MachineLearning.NNE.RDG.UnitTest.GatherHelper.Rank1Int64Indices");
	bool FGatherCPUHelperRank1Int64Indices::RunTest(const FString& Parameter)
	{
		FTensor XC4i64 = MakeConstTensorInt64(TEXT("XC4"), { 4 }, {    0,    2,    -3,     5 });
		FTensor XC4i32 = MakeConstTensorInt32(TEXT("XC4"), { 4 }, {   -1,    3,    13,     8 });
		FTensor XC4f32 = MakeConstTensor(TEXT("XC4"),      { 4 }, { 0.1f, 2.5f, -3.1f, 15.0f });

		FTensor XC3 = MakeConstTensorInt64(TEXT("XC3"), { 3 }, { 0, 2 ,3 });
		FTensor XC2 = MakeConstTensorInt64(TEXT("XC2"), { 2 }, { 1, 2 });

		//int64 Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Int64);
			CPUHelper::Gather::Apply(XC4i64, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[0]"), Y.GetPreparedData<int64>()[0], (int64)0);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[1]"), Y.GetPreparedData<int64>()[1], (int64)-3);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[2]"), Y.GetPreparedData<int64>()[2], (int64)5);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
			CPUHelper::Gather::Apply(XC4i64, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC2)[0]"), Y.GetPreparedData<int64>()[0], (int64)2);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC2)[1]"), Y.GetPreparedData<int64>()[1], (int64)-3);
		}

		//int32 Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Int32);
			CPUHelper::Gather::Apply(XC4i32, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[0]"), Y.GetPreparedData<int32>()[0], -1);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[1]"), Y.GetPreparedData<int32>()[1], 13);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[2]"), Y.GetPreparedData<int32>()[2], 8);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
			CPUHelper::Gather::Apply(XC4i32, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC2)[0]"), Y.GetPreparedData<int32>()[0], 3);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC2)[1]"), Y.GetPreparedData<int32>()[1], 13);
		}

		//float Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Float);
			CPUHelper::Gather::Apply(XC4f32, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[0]"), Y.GetPreparedData<float>()[0], 0.1f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[1]"), Y.GetPreparedData<float>()[1], -3.1f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[2]"), Y.GetPreparedData<float>()[2], 15.0f);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
			CPUHelper::Gather::Apply(XC4f32, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC2)[0]"), Y.GetPreparedData<float>()[0], 2.5f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC2)[1]"), Y.GetPreparedData<float>()[1], -3.1f);
		}

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FGatherCPUHelperRank1Int32Indices, "System.Engine.MachineLearning.NNE.RDG.UnitTest.GatherHelper.Rank1Int32Indices");
	bool FGatherCPUHelperRank1Int32Indices::RunTest(const FString& Parameter)
	{
		FTensor XC4i64 = MakeConstTensorInt64(TEXT("XC4"), { 4 }, {    0,    2,    -3,     5 });
		FTensor XC4i32 = MakeConstTensorInt32(TEXT("XC4"), { 4 }, {   -1,    3,    13,     8 });
		FTensor XC4f32 = MakeConstTensor(TEXT("XC4"),      { 4 }, { 0.1f, 2.5f, -3.1f, 15.0f });

		FTensor XC3 = MakeConstTensorInt32(TEXT("XC3"), { 3 }, { 0, 2 ,3 });
		FTensor XC2 = MakeConstTensorInt32(TEXT("XC2"), { 2 }, { 1, 2 });

		//int64 Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Int64);
			CPUHelper::Gather::Apply(XC4i64, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[0]"), Y.GetPreparedData<int64>()[0], (int64)0);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[1]"), Y.GetPreparedData<int64>()[1], (int64)-3);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC3)[2]"), Y.GetPreparedData<int64>()[2], (int64)5);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int64);
			CPUHelper::Gather::Apply(XC4i64, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC2)[0]"), Y.GetPreparedData<int64>()[0], (int64)2);
			UTEST_EQUAL(TEXT("Gather(XC4i64,XC2)[1]"), Y.GetPreparedData<int64>()[1], (int64)-3);
		}

		//int32 Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Int32);
			CPUHelper::Gather::Apply(XC4i32, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[0]"), Y.GetPreparedData<int32>()[0], -1);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[1]"), Y.GetPreparedData<int32>()[1], 13);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC3)[2]"), Y.GetPreparedData<int32>()[2], 8);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Int32);
			CPUHelper::Gather::Apply(XC4i32, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC2)[0]"), Y.GetPreparedData<int32>()[0], 3);
			UTEST_EQUAL(TEXT("Gather(XC4i32,XC2)[1]"), Y.GetPreparedData<int32>()[1], 13);
		}

		//float Data
		{
			FTensor Y = MakeTensor(TEXT("Y"), { 3 }, ENNETensorDataType::Float);
			CPUHelper::Gather::Apply(XC4f32, XC3, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[0]"), Y.GetPreparedData<float>()[0], 0.1f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[1]"), Y.GetPreparedData<float>()[1], -3.1f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC3)[2]"), Y.GetPreparedData<float>()[2], 15.0f);

			Y = MakeTensor(TEXT("Y"), { 2 }, ENNETensorDataType::Float);
			CPUHelper::Gather::Apply(XC4f32, XC2, 0, Y);
			UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC2)[0]"), Y.GetPreparedData<float>()[0], 2.5f);
			UTEST_EQUAL(TEXT("Gather(XC4f32,XC2)[1]"), Y.GetPreparedData<float>()[1], -3.1f);
		}

		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::GatherOp
