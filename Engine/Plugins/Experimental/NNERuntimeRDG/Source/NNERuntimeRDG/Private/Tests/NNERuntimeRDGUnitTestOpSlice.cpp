// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Helper/NNERuntimeRDGHelperSlice.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::SliceOp
{

#if WITH_DEV_AUTOMATION_TESTS

	namespace
	{
		void CallSliceApply(const FTensor& InputTensor, FTensor& OutputTensor, TConstArrayView<int32> Starts, TConstArrayView<int32> Steps)
		{
			CPUHelper::Slice::Apply(InputTensor, OutputTensor, Starts, Steps);
		}

		// Default steps (all 1s)
		void CallSliceApply(const FTensor& InputTensor, FTensor& OutputTensor, TConstArrayView<int32> Starts)
		{
			TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Steps;
			Steps.Init(1, InputTensor.GetShape().Rank());
			CallSliceApply(InputTensor, OutputTensor, Starts, Steps);
		}
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FSliceCPUHelperConstOuput, "System.Engine.MachineLearning.NNE.RDG.UnitTest.SliceHelper.ConstOutput");
	bool FSliceCPUHelperConstOuput::RunTest(const FString& Parameter)
	{
		FTensor XC1 = MakeConstTensor(TEXT("XC1"), { 1 }, { 1.0f });
		FTensor XC20 = MakeConstTensor(TEXT("XC20"), { 20 }, { 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f, 3.0f, 4.0f, 3.0f, 4.0f, 3.0f });
		FTensor X1 = MakeTensor(TEXT("X"), { 1 });

		FTensor Y = MakeTensor(TEXT("Y"), { 1 });
		CallSliceApply(XC1 , Y, { 0 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 1 });
		CallSliceApply(X1, Y, { 0 });
		UTEST_FALSE(TEXT("Y not const if input not const"), Y.HasPreparedData());

		Y = MakeTensor(TEXT("Y"), { 20 });
		CallSliceApply(XC20, Y, { 0 });
		UTEST_FALSE(TEXT("Y not const if input is too large "), Y.HasPreparedData());

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FSliceCPUHelperRank1, "System.Engine.MachineLearning.NNE.RDG.UnitTest.SliceHelper.Rank1");
	bool FSliceCPUHelperRank1::RunTest(const FString& Parameter)
	{
		FTensor XC6 = MakeConstTensor(TEXT("XC6"), { 6 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });

		FTensor Y = MakeTensor(TEXT("Y"), { 6 });
		CallSliceApply(XC6, Y, { 0 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[2]"), Y.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[3]"), Y.GetPreparedData<float>()[3], 4.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[4]"), Y.GetPreparedData<float>()[4], 5.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,6,0)[5]"), Y.GetPreparedData<float>()[5], 6.0f);

		Y = MakeTensor(TEXT("Y"), { 2 });
		CallSliceApply(XC6, Y, { 4 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC6,2,4)[0]"), Y.GetPreparedData<float>()[0], 5.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,2,4)[1]"), Y.GetPreparedData<float>()[1], 6.0f);

		Y = MakeTensor(TEXT("Y"), { 2 });
		CallSliceApply(XC6, Y, { 1 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC6,2,1)[0]"), Y.GetPreparedData<float>()[0], 2.0f);
		UTEST_EQUAL(TEXT("Slice(XC6,2,1)[1]"), Y.GetPreparedData<float>()[1], 3.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FSliceCPUHelperRank1Int, "System.Engine.MachineLearning.NNE.RDG.UnitTest.SliceHelper.Rank1Int");
	bool FSliceCPUHelperRank1Int::RunTest(const FString& Parameter)
	{
		FTensor XC6Int32 = MakeConstTensorInt32(TEXT("XC6Int32"), { 6 }, { 1, 2, 3, 4, 5, 6 });
		FTensor XC6Int64 = MakeConstTensorInt64(TEXT("XC6Int64"), { 6 }, { 1, 2, 3, 4, 5, 6 });
		FTensor YInt32 = MakeTensor(TEXT("YInt32"), { 2 }, ENNETensorDataType::Int32);
		FTensor YInt64 = MakeTensor(TEXT("YInt64"), { 2 }, ENNETensorDataType::Int64);

		CallSliceApply(XC6Int32, YInt32, { 1 });
		UTEST_TRUE(TEXT("YInt32 const if input is const"), YInt32.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC6Int32,2,4)[0]"), YInt32.GetPreparedData<int32>()[0], 2);
		UTEST_EQUAL(TEXT("Slice(XC6Int32,2,4)[1]"), YInt32.GetPreparedData<int32>()[1], 3);

		CallSliceApply(XC6Int64, YInt64, { 1 });
		UTEST_TRUE(TEXT("YInt64 const if input is const"), YInt64.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC6Int64,2,4)[0]"), YInt64.GetPreparedData<int64>()[0], (int64)2);
		UTEST_EQUAL(TEXT("Slice(XC6Int64,2,4)[1]"), YInt64.GetPreparedData<int64>()[1], (int64)3);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FSliceCPUHelperRank3, "System.Engine.MachineLearning.NNE.RDG.UnitTest.SliceHelper.Rank3");
	bool FSliceCPUHelperRank3::RunTest(const FString& Parameter)
	{
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });

		FTensor Y = MakeTensor(TEXT("Y"), { 1,2,3 });
		CallSliceApply(XC1x2x3, Y, { 0,0,0 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[1]"), Y.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[2]"), Y.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[3]"), Y.GetPreparedData<float>()[3], 4.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[4]"), Y.GetPreparedData<float>()[4], 5.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0-0-0)[5]"), Y.GetPreparedData<float>()[5], 6.0f);

		Y = MakeTensor(TEXT("Y"), { 1,1,3 });
		CallSliceApply(XC1x2x3, Y, { 0,1,0 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x1x3,0-1-0)[0]"), Y.GetPreparedData<float>()[0], 4.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x1x3,0-1-0)[1]"), Y.GetPreparedData<float>()[1], 5.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x1x3,0-1-0)[2]"), Y.GetPreparedData<float>()[2], 6.0f);

		Y = MakeTensor(TEXT("Y"), { 1,2,2 });
		CallSliceApply(XC1x2x3, Y, { 0,0,1 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x2,0-0-1)[0]"), Y.GetPreparedData<float>()[0], 2.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x2,0-0-1)[1]"), Y.GetPreparedData<float>()[1], 3.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x2,0-0-1)[2]"), Y.GetPreparedData<float>()[2], 5.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x2,0-0-1)[3]"), Y.GetPreparedData<float>()[3], 6.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FSliceCPUHelperSteps, "System.Engine.MachineLearning.NNE.RDG.UnitTest.SliceHelper.Steps");
	bool FSliceCPUHelperSteps::RunTest(const FString& Parameter)
	{
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });

		FTensor Y = MakeTensor(TEXT("Y"), { 1,2,2 });
		CallSliceApply(XC1x2x3, Y, { 0, 1, 0 }, { 1, -1, 2 });
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0--1-0,1--1-2)[0]"), Y.GetPreparedData<float>()[0], 4.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0--1-0,1--1-2)[1]"), Y.GetPreparedData<float>()[1], 6.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0--1-0,1--1-2)[2]"), Y.GetPreparedData<float>()[2], 1.0f);
		UTEST_EQUAL(TEXT("Slice(XC1x2x3,1x2x3,0--1-0,1--1-2)[3]"), Y.GetPreparedData<float>()[3], 3.0f);

		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::SliceOp
