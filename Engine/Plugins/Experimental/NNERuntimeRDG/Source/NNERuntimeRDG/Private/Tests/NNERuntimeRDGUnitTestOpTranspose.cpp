// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Helper/NNERuntimeRDGHelperTranspose.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

namespace UE::NNERuntimeRDG::Private::Test::TransposeOp
{

#if WITH_DEV_AUTOMATION_TESTS

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FTransposeCPUHelperIdentity, "System.Engine.MachineLearning.NNE.RDG.UnitTest.TransposeHelper.Identity");
	bool FTransposeCPUHelperIdentity::RunTest(const FString& Parameter)
	{
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });

		CPUHelper::Transpose::TransposePreparedData(XC1x2, { 0,1 });
		UTEST_EQUAL(TEXT("Transpose(XC1x2,0,1)[0]"), XC1x2.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2,0,1)[1]"), XC1x2.GetPreparedData<float>()[1], 2.0f);

		CPUHelper::Transpose::TransposePreparedData(XC1x2x3, { 0,1,2 });
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[0]"), XC1x2x3.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[1]"), XC1x2x3.GetPreparedData<float>()[1], 2.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[2]"), XC1x2x3.GetPreparedData<float>()[2], 3.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[3]"), XC1x2x3.GetPreparedData<float>()[3], 4.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[4]"), XC1x2x3.GetPreparedData<float>()[4], 5.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,0,1,2)[5]"), XC1x2x3.GetPreparedData<float>()[5], 6.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FTransposeCPUHelperReverse, "System.Engine.MachineLearning.NNE.RDG.UnitTest.TransposeHelper.Reverse");
	bool FTransposeCPUHelperReverse::RunTest(const FString& Parameter)
	{
		FTensor XC1x2 = MakeConstTensor(TEXT("XC1x2"), { 1,2 }, { 1.0f, 2.0f });
		FTensor XC1x2x3 = MakeConstTensor(TEXT("XC1x2x3"), { 1,2,3 }, { 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f });

		CPUHelper::Transpose::TransposePreparedData(XC1x2, { 1,0 });
		UTEST_EQUAL(TEXT("Transpose(XC1x2,1,0)[0]"), XC1x2.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2,1,0)[1]"), XC1x2.GetPreparedData<float>()[1], 2.0f);

		CPUHelper::Transpose::TransposePreparedData(XC1x2x3, { 2,1,0 });
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[0]"), XC1x2x3.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[1]"), XC1x2x3.GetPreparedData<float>()[1], 4.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[2]"), XC1x2x3.GetPreparedData<float>()[2], 2.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[3]"), XC1x2x3.GetPreparedData<float>()[3], 5.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[4]"), XC1x2x3.GetPreparedData<float>()[4], 3.0f);
		UTEST_EQUAL(TEXT("Transpose(XC1x2x3,2,1,0)[5]"), XC1x2x3.GetPreparedData<float>()[5], 6.0f);

		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FTransposeCPUHelperConvKernel, "System.Engine.MachineLearning.NNE.RDG.UnitTest.TransposeHelper.ConvKernel");
	bool FTransposeCPUHelperConvKernel::RunTest(const FString& Parameter)
	{
		FTensor XC2x3x4x5 = MakeConstTensor(TEXT("XC2x3x4x5"), { 2,3,4,5 }, { 
			0.0f,   1.0f,   2.0f,   3.0f,   4.0f,   5.0f,   6.0f,   7.0f,   8.0f,   9.0f,   10.0f,  11.0f,  12.0f,
			13.0f,  14.0f,  15.0f,  16.0f,  17.0f,  18.0f,  19.0f,  20.0f,  21.0f,  22.0f,  23.0f,  24.0f,  25.0f,
			26.0f,  27.0f,  28.0f,  29.0f,  30.0f,  31.0f,  32.0f,  33.0f,  34.0f,  35.0f,  36.0f,  37.0f,  38.0f,
			39.0f,  40.0f,  41.0f,  42.0f,  43.0f,  44.0f,  45.0f,  46.0f,  47.0f,  48.0f,  49.0f,  50.0f,  51.0f,
			52.0f,  53.0f,  54.0f,  55.0f,  56.0f,  57.0f,  58.0f,  59.0f,  60.0f,  61.0f,  62.0f,  63.0f,  64.0f,
			65.0f,  66.0f,  67.0f,  68.0f,  69.0f,  70.0f,  71.0f,  72.0f,  73.0f,  74.0f,  75.0f,  76.0f,  77.0f,
			78.0f,  79.0f,  80.0f,  81.0f,  82.0f,  83.0f,  84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f,  90.0f,
			91.0f,  92.0f,  93.0f,  94.0f,  95.0f,  96.0f,  97.0f,  98.0f,  99.0f, 100.0f, 101.0f, 102.0f, 103.0f,
			104.0f,105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f,
			117.0f,118.0f, 119.0f});

		TArray<float> ExpectedResult({
			0.0f,   60.0f,  20.0f,  80.0f,  40.0f, 100.0f,   1.0f,  61.0f,  21.0f,  81.0f,  41.0f, 101.0f,   2.0f,
			62.0f,  22.0f,  82.0f,  42.0f, 102.0f,   3.0f,  63.0f,  23.0f,  83.0f,  43.0f, 103.0f,   4.0f,  64.0f,
			24.0f,  84.0f,  44.0f, 104.0f,   5.0f,  65.0f,  25.0f,  85.0f,  45.0f, 105.0f,   6.0f,  66.0f,  26.0f,
			86.0f,  46.0f, 106.0f,   7.0f,  67.0f,  27.0f,  87.0f,  47.0f, 107.0f,   8.0f,  68.0f,  28.0f,  88.0f,
			48.0f, 108.0f,   9.0f,  69.0f,  29.0f,  89.0f,  49.0f, 109.0f,  10.0f,  70.0f,  30.0f,  90.0f,  50.0f,
			110.0f, 11.0f,  71.0f,  31.0f,  91.0f,  51.0f, 111.0f,  12.0f,  72.0f,  32.0f,  92.0f,  52.0f, 112.0f,
			13.0f,  73.0f,  33.0f,  93.0f,  53.0f, 113.0f,  14.0f,  74.0f,  34.0f,  94.0f,  54.0f, 114.0f,  15.0f,
			75.0f,  35.0f,  95.0f,  55.0f, 115.0f,  16.0f,  76.0f,  36.0f,  96.0f,  56.0f, 116.0f,  17.0f,  77.0f,
			37.0f,  97.0f,  57.0f, 117.0f,  18.0f,  78.0f,  38.0f,  98.0f,  58.0f, 118.0f,  19.0f,  79.0f,  39.0f,
			99.0f,  59.0f, 119 });

		CPUHelper::Transpose::TransposePreparedData(XC2x3x4x5, { 2,3,1,0 });
		for (uint32 i = 0; i < XC2x3x4x5.GetVolume(); ++i)
		{
			FString TestText = FString::Printf(TEXT("Transpose(XC2x3x4x5,2,3,1,0)[ %d]"), i);
			UTEST_EQUAL(*TestText, XC2x3x4x5.GetPreparedData<float>()[i], ExpectedResult[i]);
		}

		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::TransposeOp
