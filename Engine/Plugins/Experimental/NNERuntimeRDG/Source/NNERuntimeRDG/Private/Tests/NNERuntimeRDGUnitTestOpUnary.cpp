// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/EnumRange.h"

#include "Helper/NNERuntimeRDGHelperElementWiseUnary.h"
#include "NNERuntimeRDGUnitTestHelper.h"
#include "NNERuntimeRDGTensor.h"
#include "NNETypes.h"
#include <limits.h>

#if WITH_DEV_AUTOMATION_TESTS
#include "Misc/AutomationTest.h"
#endif

ENUM_RANGE_BY_COUNT(UE::NNEHlslShaders::Internal::EElementWiseUnaryOperatorType, UE::NNEHlslShaders::Internal::EElementWiseUnaryOperatorType::MAX)

namespace UE::NNERuntimeRDG::Private::Test::UnaryOp
{

#if WITH_DEV_AUTOMATION_TESTS

	using namespace NNE;
	using EElementWiseUnaryOperatorType = NNEHlslShaders::Internal::EElementWiseUnaryOperatorType;

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperConstOuput, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.ConstOutput");
	bool FElementWiseUnaryCPUHelperConstOuput::RunTest(const FString& Parameter)
	{
		for (EElementWiseUnaryOperatorType Op : TEnumRange<EElementWiseUnaryOperatorType>())
		{
			if (!TestUnaryOutputIsOnlyComputedWhenItShould(Op))
			{
				return false;
			}
		}
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAbs, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Abs");
	bool FElementWiseUnaryCPUHelperAbs::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -1.0f, 2.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Abs, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Abs(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Abs(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Abs(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Abs(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAcos, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Acos");
	bool FElementWiseUnaryCPUHelperAcos::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Acos, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Acos(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Acos(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Acos(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Acos(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAcosh, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Acosh");
	bool FElementWiseUnaryCPUHelperAcosh::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, 1.5f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Acosh, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Acosh(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.0f);
		UTEST_EQUAL(TEXT("Acosh(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.9624236501192069f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAsin, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Asin");
	bool FElementWiseUnaryCPUHelperAsin::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Asin, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Asin(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Asin(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Asin(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Asin(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAsinh, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Asinh");
	bool FElementWiseUnaryCPUHelperAsinh::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, 2.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Asinh, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Asinh(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.881373587019543f);
		UTEST_EQUAL(TEXT("Asinh(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.4436354751788103f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperAtan, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Atan");
	bool FElementWiseUnaryCPUHelperAtan::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Atan, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Atan(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Atan(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Atan(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Atan(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperCeil, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Ceil");
	bool FElementWiseUnaryCPUHelperCeil::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Ceil, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Ceil(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::CeilToFloat(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Ceil(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::CeilToFloat(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperClip, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Clip");
	bool FElementWiseUnaryCPUHelperClip::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC3x1"), { 3,1 }, { 0.5f, 1.8f, 2.1f });
		FTensor Y = MakeTensor(TEXT("Y"), { 3,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Clip, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Clip(XC3x1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("Clip(XC3x1)[1]"), Y.GetPreparedData<float>()[1], XC2x1.GetPreparedData<float>()[1]);
		UTEST_EQUAL(TEXT("Clip(XC3x1)[2]"), Y.GetPreparedData<float>()[2], 2.0f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperCos, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Cos");
	bool FElementWiseUnaryCPUHelperCos::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Cos, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cos(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Cos(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Cos(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Cos(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperCosh, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Cosh");
	bool FElementWiseUnaryCPUHelperCosh::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, 2.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Cosh, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Cosh(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 1.5430806348152437f);
		UTEST_EQUAL(TEXT("Cosh(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 3.7621956910836314f);
		return true;
	}
	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperElu, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Elu");
	bool FElementWiseUnaryCPUHelperElu::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Elu, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Elu(XC2x1)[0]"), Y.GetPreparedData<float>()[0], -0.3934693402873666f);
		UTEST_EQUAL(TEXT("Elu(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.8f);
		return true;
	}
	
	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperErf, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Erf");
	bool FElementWiseUnaryCPUHelperErf::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, 2.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Erf, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL_TOLERANCE(TEXT("Erf(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.8427007929497149f, 1e-3f);
		UTEST_EQUAL_TOLERANCE(TEXT("Erf(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.9953222650189527f, 1e-3f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperExp, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Exp");
	bool FElementWiseUnaryCPUHelperExp::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Exp, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Exp(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Exp(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Exp(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Exp(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperFloor, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Floor");
	bool FElementWiseUnaryCPUHelperFloor::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Floor, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Floor(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Floor(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Floor(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Floor(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}
	
	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperIsInf, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.IsInf");
	bool FElementWiseUnaryCPUHelperIsInf::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, std::numeric_limits<float>::infinity() });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::IsInf, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("IsInf(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.0f);
		UTEST_EQUAL(TEXT("IsInf(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.0f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperIsNan, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.IsNan");
	bool FElementWiseUnaryCPUHelperIsNan::RunTest(const FString& Parameter)
	{
		float FloatNan = FMath::Sqrt(-1.0f);
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, FloatNan });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::IsNan, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("IsNan(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.0f);
		UTEST_EQUAL(TEXT("IsNan(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.0f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperHardSigmoid, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.HardSigmoid");
	bool FElementWiseUnaryCPUHelperHardSigmoid::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.0f, -1.5f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::HardSigmoid, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("HardSigmoid(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("HardSigmoid(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.5f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperHardswitch, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.HardSwitch");
	bool FElementWiseUnaryCPUHelperHardswitch::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.0f, 1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::HardSwish, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("HardSwitch(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.5f);
		UTEST_EQUAL(TEXT("HardSwitch(XC2x1)[1]"), Y.GetPreparedData<float>()[1], (1.0f / 6.0f) + 0.5f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperLeakyRelu, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.LeakyRelu");
	bool FElementWiseUnaryCPUHelperLeakyRelu::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, -1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::LeakyRelu, XC2x1, -0.5f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("LeakyRelu(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 1.0f);
		UTEST_EQUAL(TEXT("LeakyRelu(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.5f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperLog, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Log");
	bool FElementWiseUnaryCPUHelperLog::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Log, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Log(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Loge(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Log(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Loge(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperNeg, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Neg");
	bool FElementWiseUnaryCPUHelperNeg::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Neg, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Neg(XC2x1)[0]"), Y.GetPreparedData<float>()[0], -XC2x1.GetPreparedData<float>()[0]);
		UTEST_EQUAL(TEXT("Neg(XC2x1)[1]"), Y.GetPreparedData<float>()[1], -XC2x1.GetPreparedData<float>()[1]);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperReciprocal, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Reciprocal");
	bool FElementWiseUnaryCPUHelperReciprocal::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Reciprocal, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Reciprocal(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 1.0f / (XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Reciprocal(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.0f / (XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperRelu, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Relu");
	bool FElementWiseUnaryCPUHelperRelu::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Relu, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Relu(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.0f);
		UTEST_EQUAL(TEXT("Relu(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.8f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSelu, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Selu");
	bool FElementWiseUnaryCPUHelperSelu::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.1f, -1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Selu, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Selu(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.3f);
		UTEST_EQUAL(TEXT("Selu(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 3.0f * (0.36787944117144233f - 1.0f));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSigmoid, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Sigmoid");
	bool FElementWiseUnaryCPUHelperSigmoid::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.0f, 1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Sigmoid, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Sigmoid(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.5f);
		UTEST_EQUAL(TEXT("Sigmoid(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.0f / (1.0f + 0.36787944117144233f));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSign, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Sign");
	bool FElementWiseUnaryCPUHelperSign::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Sign, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Sign(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Sign(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Sign(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Sign(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSin, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Sin");
	bool FElementWiseUnaryCPUHelperSin::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Sin, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Sin(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Sin(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Sin(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Sin(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSinh, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Sinh");
	bool FElementWiseUnaryCPUHelperSinh::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Sinh, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Sinh(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Sinh(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Sinh(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Sinh(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSoftplus, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Softplus");
	bool FElementWiseUnaryCPUHelperSoftplus::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.0f, 1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Softplus, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Softplus(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.6931471805599453f);
		UTEST_EQUAL(TEXT("Softplus(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 1.3132616875182228f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSoftsign, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Softsign");
	bool FElementWiseUnaryCPUHelperSoftsign::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 0.0f, 1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Softsign, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Softsign(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.0f);
		UTEST_EQUAL(TEXT("Softsign(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.5f);
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperSqrt, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Sqrt");
	bool FElementWiseUnaryCPUHelperSqrt::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 2.0f, 1.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Sqrt, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Sqrt(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Sqrt(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Sqrt(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Sqrt(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperTan, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Tan");
	bool FElementWiseUnaryCPUHelperTan::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { -0.5f, 0.8f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Tan, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Tan(XC2x1)[0]"), Y.GetPreparedData<float>()[0], FMath::Tan(XC2x1.GetPreparedData<float>()[0]));
		UTEST_EQUAL(TEXT("Tan(XC2x1)[1]"), Y.GetPreparedData<float>()[1], FMath::Tan(XC2x1.GetPreparedData<float>()[1]));
		return true;
	}

	IMPLEMENT_NNE_SHAPEINFERENCEHELPER_UNIT_AUTOMATION_TEST(FElementWiseUnaryCPUHelperTanh, "System.Engine.MachineLearning.NNE.RDG.UnitTest.UnaryHelper.Tanh");
	bool FElementWiseUnaryCPUHelperTanh::RunTest(const FString& Parameter)
	{
		FTensor XC2x1 = MakeConstTensor(TEXT("XC2x1"), { 2,1 }, { 1.0f, 2.0f });
		FTensor Y = MakeTensor(TEXT("Y"), { 2,1 });
		CPUHelper::ElementWiseUnary::Apply(EElementWiseUnaryOperatorType::Tanh, XC2x1, 1.0f, 2.0f, 3.0f, Y);
		UTEST_TRUE(TEXT("Y const if input is const"), Y.HasPreparedData());
		UTEST_EQUAL(TEXT("Tanh(XC2x1)[0]"), Y.GetPreparedData<float>()[0], 0.7615941559557649f);
		UTEST_EQUAL(TEXT("Tanh(XC2x1)[1]"), Y.GetPreparedData<float>()[1], 0.9640275800758169f);
		return true;
	}

#endif //WITH_DEV_AUTOMATION_TESTS

} // namespace UE::NNERuntimeRDG::Private::Test::UnaryOp
