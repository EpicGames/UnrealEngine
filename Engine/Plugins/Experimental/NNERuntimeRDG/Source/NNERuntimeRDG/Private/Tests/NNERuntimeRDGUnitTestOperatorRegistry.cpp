// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "NNERuntimeRDGBase.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace UE::NNERuntimeRDG::Private::Test::Registry
{

// Base class for testing operator registry 
class FOperatorTest
{
public:
	virtual ~FOperatorTest() = default;
};

// Test operator that has single version
class FOperatorTestConv : public FOperatorTest
{
public:
	static FOperatorTest* Create()
	{
		return new FOperatorTestConv();
	}
	static bool Validate(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		return true;
	}
};

// Test operator that has multiple versions
template<uint32 OperatorVersion>
class FOperatorTestClip : public FOperatorTest
{
public:
	static FOperatorTest* Create()
	{
		return new FOperatorTestClip();
	}
	static bool Validate(const NNERuntimeRDGData::Internal::FAttributeMap& AttributeMap, TConstArrayView<ENNETensorDataType> InputTypes, TConstArrayView<NNE::FSymbolicTensorShape> InputShapes)
	{
		return true;
	}
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FOperatorRegistryRDGTest,
	"System.Engine.MachineLearning.NNE.RDG.UnitTest.OperatorRegistry",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FOperatorRegistryRDGTest::RunTest(const FString&)
{
    using FRegistryTest = TOperatorRegistryRDG<FOperatorTest>;

    struct FOpRegistrationRef
    {
        FOperatorDesc           OpDesc;
        FRegistryTest::OperatorCreateFunc 		CreateFunc;
		FRegistryTest::OperatorValidateFunc 	ValidateFunc;
    };

    TArray<FOpRegistrationRef> TestOps;

#define OP_TEST_CLASS(OpName) FOperatorTest##OpName
#define OP_TEST_CLASS_TEMPL(OpName,OpVer) OP_TEST_CLASS(OpName) <OpVer>

// Register operator with default version
#define OP_REG(OpName) \
{ \
    FOperatorDesc OpDesc {{TEXT(#OpName), TEXT("Onnx")}}; \
	FRegistryTest::Get()->OpAdd(OpDesc, OP_TEST_CLASS(OpName)::Create, OP_TEST_CLASS(OpName)::Validate); \
	TestOps.Add({OpDesc, OP_TEST_CLASS(OpName)::Create, OP_TEST_CLASS(OpName)::Validate}); \
} \
const int Op##OpName##Idx = TestOps.Num() - 1;
// Register operator with version
#define OP_REG_VER(OpName, OpVer) \
{ \
    FOperatorDesc OpDesc {{TEXT(#OpName), TEXT("Onnx")}, OpVer}; \
	FRegistryTest::Get()->OpAdd(OpDesc, OP_TEST_CLASS_TEMPL(OpName,OpVer)::Create, OP_TEST_CLASS_TEMPL(OpName,OpVer)::Validate); \
	TestOps.Add({OpDesc, OP_TEST_CLASS_TEMPL(OpName,OpVer)::Create, OP_TEST_CLASS_TEMPL(OpName,OpVer)::Validate}); \
} \
const int Op##OpName##OpVer##Idx = TestOps.Num() - 1;

	bool bPrevSuppressLogWarnings = bSuppressLogWarnings;
    bSuppressLogWarnings = true;

	OP_REG(Conv)
	OP_REG_VER(Clip, 9)
	OP_REG_VER(Clip, 11)
	OP_REG_VER(Clip, 19)
	
	// Test registration
	for (int32 Idx = 0; Idx < TestOps.Num(); ++Idx)
	{
		FRegistryTest::OperatorCreateFunc CreateFunc = FRegistryTest::Get()->OpFind(TestOps[Idx].OpDesc);
		UTEST_NOT_NULL(TEXT("OpFind()"), (void *) CreateFunc);
        UTEST_EQUAL(TEXT("OpFoundMatches"), CreateFunc, TestOps[Idx].CreateFunc);
		FOperatorTest* Op = CreateFunc();
		UTEST_NOT_NULL(TEXT("OpCreateCall"), Op);
	}
	// Test validation
	for (int32 Idx = 0; Idx < TestOps.Num(); ++Idx)
	{
		FRegistryTest::OperatorValidateFunc ValidateFunc = FRegistryTest::Get()->OpFindValidation(TestOps[Idx].OpDesc);
		UTEST_NOT_NULL(TEXT("OpFindValidation()"), (void *) ValidateFunc);
        UTEST_EQUAL(TEXT("OpFoundMatches"), ValidateFunc, TestOps[Idx].ValidateFunc);
        UE::NNERuntimeRDGData::Internal::FAttributeMap					Attributes;
		TArray<ENNETensorDataType>				InputTypes;
		TArray<UE::NNE::FSymbolicTensorShape>	InputShapes;
		UTEST_TRUE(TEXT("OpValidateCall"), ValidateFunc(Attributes, InputTypes, InputShapes));
	}

    UTEST_EQUAL(TEXT("Find3InConvUnversioned"), 
        FRegistryTest::Get()->OpFind({{TEXT("Conv"), TEXT("Onnx")}, 3}), 
        TestOps[OpConvIdx].CreateFunc
    );

    
    UTEST_NULL(TEXT("FindUnversionedInClipVersioned"), 
        (void *) FRegistryTest::Get()->OpFind({{TEXT("Clip"), TEXT("Onnx")}})
    );
	
    bSuppressLogWarnings = bPrevSuppressLogWarnings;

	return true;
}

} // namespace UE::NNERuntimeRDG::Private::Test::Registry
#endif