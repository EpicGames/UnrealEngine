// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "NNERuntimeRDGDataAttributeDataType.generated.h"

/**
 * Attribute data types
 * 
 * Note: also extend NNERuntimeRDGDataAttributeValueTraits.h for more type support
 */
UENUM()
enum class ENNERuntimeRDGDataAttributeDataType : uint8
{
	None,
	Float,								//!< 32-bit floating number
	FloatArray,							//!< TArray of 32-bit floating numbers
	Int32,								//!< 32-bit signed integer
	Int32Array,							//!< TArray of 32-bit signed integers
	String,								//!< built-in FString
	StringArray,						//!< TArray of built-in FString
	Tensor,								//!< FAttributeTensor
	TensorArray							//!< TArray of FAttributeTensor
};