// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGDataAttributeValue.h"

namespace UE::NNERuntimeRDGData::Internal
{

class FAttributeMap
{
public:
	//Set attribute
	void SetAttribute(const FString& Name, const FNNERuntimeRDGDataAttributeValue& Value)
	{
#if DO_CHECK
		const bool bIsUnique = (nullptr == Attributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; }));
		checkf(bIsUnique, TEXT("Attribute name should be unique"));
#endif
		Attributes.Emplace(Name, Value);
	}

	template<typename T>
	T GetValue(const FString& Name) const
	{
		const FNNERuntimeRDGDataAttributeValue* Value = GetAttributeValue(Name);
		checkf(Value != nullptr, TEXT("Required attribute %s not found"), *Name);
		return Value->GetValue<T>();
	}

	//Query attributes
	template<typename T>
	T GetValueOrDefault(const FString& Name, T Default) const
	{
		const FNNERuntimeRDGDataAttributeValue* Value = GetAttributeValue(Name);
		return Value == nullptr ? Default : Value->GetValue<T>();
	}
	
	const FNNERuntimeRDGDataAttributeValue* GetAttributeValue(const FString& Name) const
	{
		const FEntry * entry = Attributes.FindByPredicate([Name](const FEntry& Other) { return Other.Name == Name; });
		if (entry != nullptr)
		{
			return &entry->Value;
		}
		return nullptr;
	}

	//Iteration
	int32 Num() const
	{
		return Attributes.Num();
	}

	const FString& GetName(int32 Idx) const
	{
		check(Idx < Num());
		return Attributes[Idx].Name;
	}

	const FNNERuntimeRDGDataAttributeValue& GetAttributeValue(int32 Idx) const
	{
		check(Idx < Num());
		return Attributes[Idx].Value;
	}

private:

	struct FEntry
	{
		FEntry(const FString& InName, const FNNERuntimeRDGDataAttributeValue& InValue)
			: Name(InName), Value(InValue)
		{
		}
		
		FString Name;
		FNNERuntimeRDGDataAttributeValue Value;
	};

	TArray<FEntry> Attributes;
};

} // namespace UE::NNERuntimeRDGData::Internal