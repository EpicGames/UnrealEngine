// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "NNETypes.h"
#include "Serialization/Archive.h"

namespace UE::NNERuntimeRDGData::Internal
{
	class FAttributeTensor
	{
	protected:
		ENNETensorDataType	DataType;
		TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> Shape;
		TArray<uint8>		Data;

	public:

		ENNETensorDataType GetDataType() const
		{
			return DataType;
		}

		TConstArrayView<uint8> GetData() const
		{
			return Data;
		}

		NNE::FTensorShape GetShape() const
		{
			return NNE::FTensorShape::Make(Shape);
		}

		static FAttributeTensor Make(const NNE::FTensorShape& Shape,
									 ENNETensorDataType DataType, 
									 TConstArrayView<uint8> Data)
		{
			uint64 Volume = Shape.Volume();
			check(Volume <= TNumericLimits<uint32>::Max());
			check(Data.NumBytes() == NNE::GetTensorDataTypeSizeInBytes(DataType) * Volume);

			FAttributeTensor Tensor;
			Tensor.DataType = DataType;
			Tensor.Shape.Reset();
			Tensor.Shape.Append(Shape.GetData());
			Tensor.Data.Reset();
			Tensor.Data.Append(Data);
			return Tensor;
		}

		friend FArchive& operator<<(FArchive& Ar, FAttributeTensor& Tensor)
		{
			Ar << Tensor.DataType;
			Ar << Tensor.Shape;
			Ar << Tensor.Data;
			return Ar;
		}
	};
} // namespace UE::RuntimeRDGData::Internal
