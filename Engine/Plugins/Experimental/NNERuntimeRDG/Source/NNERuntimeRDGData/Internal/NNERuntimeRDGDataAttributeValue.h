// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGDataAttributeValueTraits.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/Object.h"

#include "NNERuntimeRDGDataAttributeValue.generated.h"

USTRUCT()
struct FNNERuntimeRDGDataAttributeValue
{
	GENERATED_USTRUCT_BODY()

	FNNERuntimeRDGDataAttributeValue()
	{
		Type = ENNERuntimeRDGDataAttributeDataType::None;
	}

	template<typename T>
	explicit FNNERuntimeRDGDataAttributeValue(T InValue)
	{
		FMemoryWriter writer(Value, /*bIsPersitent =*/ true);
		writer << InValue;

		Type = TNNERuntimeRDGDataAttributeValueTraits<T>::GetType();
	}

	template<typename T>
	T GetValue() const
	{
		check((Type == TNNERuntimeRDGDataAttributeValueTraits<T>::GetType()));

		T Result;

		FMemoryReader Reader(Value, /*bIsPersitent =*/ true);
		Reader << Result;

		return Result;
	}

	ENNERuntimeRDGDataAttributeDataType GetType() const
	{
		return Type;
	}
	
private:
	
	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	ENNERuntimeRDGDataAttributeDataType	Type;
	
	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<uint8>			Value;
};