// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGDataAttributeDataType.h"
#include "NNERuntimeRDGDataAttributeTensor.h"

template<typename T> struct TNNERuntimeRDGDataAttributeValueTraits
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType()
	{
		static_assert(!sizeof(T), "Attribute value trait must be specialized for this type.");
		return ENNERuntimeRDGDataAttributeDataType::None;
	}
};

// Attribute specializations
template<> struct TNNERuntimeRDGDataAttributeValueTraits<float>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::Float; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<int32>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::Int32; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<FString>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::String; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<UE::NNERuntimeRDGData::Internal::FAttributeTensor>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::Tensor; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<TArray<int32>>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::Int32Array; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<TArray<float>>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::FloatArray; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<TArray<FString>>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::StringArray; }
};

template<> struct TNNERuntimeRDGDataAttributeValueTraits<TArray<UE::NNERuntimeRDGData::Internal::FAttributeTensor>>
{
	static constexpr ENNERuntimeRDGDataAttributeDataType GetType() { return ENNERuntimeRDGDataAttributeDataType::TensorArray; }
};
