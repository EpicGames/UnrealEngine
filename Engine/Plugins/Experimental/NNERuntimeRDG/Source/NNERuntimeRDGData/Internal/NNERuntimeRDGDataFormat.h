// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGDataAttributeValue.h"
#include "NNETypes.h"
#include "UObject/Object.h"

#include "NNERuntimeRDGDataFormat.generated.h"

UENUM()
enum class ENNERuntimeRDGDataTensorType : uint8
{
	None,	
	Input,
	Output,
	Intermediate,
	Initializer,
	Empty,

	NUM
};

USTRUCT()
struct FNNERuntimeRDGDataAttributeDesc
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	FNNERuntimeRDGDataAttributeValue Value;
};

USTRUCT()
struct FNNERuntimeRDGDataOperatorDesc
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	FString TypeName;			//!< For example "Relu"

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	FString DomainName;			//!< For example "onnx"

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TOptional<uint32> Version;	//!< For example 7

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<uint32> InTensors;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<uint32> OutTensors;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<FNNERuntimeRDGDataAttributeDesc> Attributes;
};

USTRUCT()
struct FNNERuntimeRDGDataTensorDesc
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	FString Name;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<int32> Shape;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	ENNERuntimeRDGDataTensorType	Type = ENNERuntimeRDGDataTensorType::None;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	ENNETensorDataType	DataType = ENNETensorDataType::None;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	uint64	DataSize = 0;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	uint64	DataOffset = 0;
};

USTRUCT()
struct FNNERuntimeRDGDataModelFormat
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<FNNERuntimeRDGDataTensorDesc> Tensors;

	UPROPERTY(VisibleAnywhere, Category = "Neural Network Inference")
	TArray<FNNERuntimeRDGDataOperatorDesc> Operators;

	uint64 DataSize;
	TArray64<uint8> TensorData;
	
	bool Serialize(FArchive& Ar)
	{
		// Serialize normal UPROPERTY tagged data
		UScriptStruct* Struct = FNNERuntimeRDGDataModelFormat::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), Struct, nullptr);

		if (Ar.IsLoading())
		{
			Ar << DataSize;
			TensorData.SetNumUninitialized(DataSize);
			Ar.Serialize((void*)TensorData.GetData(), DataSize);
		}
		else if (Ar.IsSaving())
		{
			DataSize = TensorData.Num();
			Ar << DataSize;
			Ar.Serialize((void*)TensorData.GetData(), DataSize);
		}

		return true;
	}

};

template<>
struct TStructOpsTypeTraits<FNNERuntimeRDGDataModelFormat> : public TStructOpsTypeTraitsBase2<FNNERuntimeRDGDataModelFormat>
{
	enum
	{
		WithSerializer = true
	};
};