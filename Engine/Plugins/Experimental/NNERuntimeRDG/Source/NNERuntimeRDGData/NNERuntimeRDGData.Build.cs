// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class NNERuntimeRDGData : ModuleRules
{
	public NNERuntimeRDGData(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.Add("NNE");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);
	}
}

