// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGUtilsModelOptimizerInterface.h"

namespace UE::NNERuntimeRDGUtils::Internal
{

NNERUNTIMERDGUTILS_API TUniquePtr<IModelOptimizer> CreateModelOptimizer();

} // UE::NNERuntimeRDGUtils::Internal
