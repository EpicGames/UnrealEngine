// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::NNERuntimeRDGUtils::Internal
{
	
/** Interface class for NNE model validator */
class IModelValidator
{
public:
	virtual ~IModelValidator() = default;
	virtual FString GetName() const = 0;
	virtual bool ValidateModel(TConstArrayView<uint8> InputModel) const = 0;
};

/** Interface class for NNE model optimizer pass */
class IModelOptimizerPass
{
public:
	virtual ~IModelOptimizerPass() = default;
	virtual FString GetName() const = 0;

	//Optimize the model in place, potentially changing the format
	virtual bool ApplyPass(TArray<uint8>& Model) const = 0;
};

/** Interface class for NNE model optimizer */
class IModelOptimizer
{
public:
	virtual ~IModelOptimizer() = default;
	virtual FString GetName() const = 0;

	//Allow to extend/customize an optimizer by adding passes. They should be executed in order.
	virtual void AddOptimizationPass(TSharedPtr<IModelOptimizerPass> ModelOptimizerPass) = 0;
	
	//Allow to extend/customize an optimizer all validators should be run between each pass.
	virtual void AddValidator(TSharedPtr<IModelValidator>) = 0;
	
	//Apply all passes and validators to the input model, produce an optimized model potentially in a different format
	virtual bool Optimize(TConstArrayView<uint8> InputModel, TArray<uint8>& OutModel) = 0;
};

} // namespace UE::NNERuntimeRDGUtils::Internal
