// Copyright Epic Games, Inc. All Rights Reserved.


using UnrealBuildTool;
using System.IO;

public class NNERuntimeRDGUtils : ModuleRules
{
	public NNERuntimeRDGUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.Add("NNE");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"NNEHlslShaders",
				"NNERuntimeRDGData",
				"NNERuntimeRDGProtobufEditor",
				"NNERuntimeRDGOnnxruntimeEditor",
				"NNERuntimeRDGOnnxEditor"
			}
		);
	}
}

