// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsHelpers.h"

#include "NNEHlslShadersLog.h"

THIRD_PARTY_INCLUDES_START
#include <onnx/defs/schema.h>
THIRD_PARTY_INCLUDES_END


namespace UE::NNERuntimeRDGUtils::Private
{

TOptional<uint32> GetOpVersionFromOpsetVersion(const FString& OpType, int OpsetVersion)
{
	const onnx::OpSchema* OpSchema = onnx::OpSchemaRegistry::Schema(TCHAR_TO_ANSI(*OpType), OpsetVersion);
	if(OpSchema == nullptr)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "No OpSchema found for operator %ls and OpSet version %d.", *OpType, OpsetVersion);
		return TOptional<uint32>();
	}
	return (uint32) OpSchema->SinceVersion();
}

} // namespace UE::NNERuntimeRDGUtils::Private