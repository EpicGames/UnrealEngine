// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OptionalFwd.h"
#include "NNETypes.h"
#include "Templates/UniquePtr.h"

struct FNNERuntimeRDGDataAttributeValue;

namespace UE::NNERuntimeRDGUtils::Private
{

class IModelBuilder
{
public:

	enum class EHandleType : uint8
	{
		Invalid,
		Tensor,
		Operator
	};

	template<EHandleType Tag>
	struct THandle
	{
		void*		Ptr { nullptr };
		EHandleType	Type { Tag };
	};

	typedef struct THandle<EHandleType::Tensor>		FHTensor;
	typedef struct THandle<EHandleType::Operator>	FHOperator;

	template<EHandleType Tag>
	THandle<Tag> MakeHandle(void* Ptr)
	{
		THandle<Tag> Handle;
		Handle.Ptr = Ptr;
		check(Handle.Type != EHandleType::Invalid);
		return Handle;
	}

	virtual ~IModelBuilder() = default;

	// Initialize the model builder
	virtual bool Begin(const FString& GraphName = TEXT("DefaultGraphName")) = 0;

	// Serialize the model to given array
	virtual bool End(TArray<uint8>& Data) = 0;

	virtual FHTensor AddTensor(const FString& Name, ENNETensorDataType DataType, TArrayView<const int32> Shape) = 0;
	virtual FHTensor AddConstantTensor(const FString& Name, ENNETensorDataType DataType, TArrayView<const int32> Shape, const void* Data, uint64 DataSize) = 0;
	virtual FHTensor AddEmptyTensor() = 0;
	virtual bool AddInput(FHTensor InTensor) = 0;
	virtual bool AddOutput(FHTensor OutTensor) = 0;
	virtual FHOperator AddOperator(const FString& Type, const FString& Domain, TOptional<uint32> Version = FNullOpt{0}, const FString& Name = TEXT("")) = 0;
	virtual bool AddOperatorInput(FHOperator Op, FHTensor Tensor) = 0;
	virtual bool AddOperatorAttribute(FHOperator Op, const FString& Name, const FNNERuntimeRDGDataAttributeValue& Value) = 0;
	virtual bool AddOperatorOutput(FHOperator Op, FHTensor Tensor) = 0;
};

static constexpr TCHAR OnnxDomainName[] = TEXT("Onnx");

} // UE::NNERuntimeRDGUtils::Private

