// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsModelBuilderNNE.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeMap.h"
#include "NNERuntimeRDGDataFormat.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryWriter.h"

namespace UE::NNERuntimeRDGUtils::Private
{

namespace ModelBuilderNNEHelper
{
	int32 NNETensorCast(IModelBuilder::FHTensor& Handle)
	{
		if (Handle.Type == IModelBuilder::EHandleType::Tensor)
		{
			return int32(reinterpret_cast<int64>(Handle.Ptr));
		}

		return -1;
	}

	int32 NNEOperatorCast(IModelBuilder::FHOperator& Handle)
	{
		if (Handle.Type == IModelBuilder::EHandleType::Operator)
		{
			return int32(reinterpret_cast<int64>(Handle.Ptr));
		}

		return -1;
	}
} //namespace ModelBuilderNNEHelper

class FModelBuilderNNE : public IModelBuilder
{
public:
	virtual bool Begin(const FString& Name) override
	{
		return true;
	}

	virtual bool End(TArray<uint8>& Data) override
	{
		FMemoryWriter Writer(Data, /*bIsPersitent =*/ true);
		
		Format.Serialize(Writer);

		return !Data.IsEmpty();
	}

	virtual FHTensor AddTensor(const FString& Name, ENNETensorDataType DataType, TArrayView<const int32> Shape) override
	{
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> NNEShape;
		for (int32 i = 0; i < Shape.Num(); ++i)
		{
			//Allow caller to use 0 for variable dimensions for inputs/outputs, NNE use -1.
			//RDG not supporting 0 sized dimension at the moment.
			NNEShape.Emplace(Shape[i] == 0 ? -1 : Shape[i]);
		}

		int32 Idx = AddTensor(GetBaseTensorDesc(Name, NNEShape, DataType));

		return MakeHandle<EHandleType::Tensor>(reinterpret_cast<void*>((int64)Idx));
	}

	virtual FHTensor AddConstantTensor(const FString& Name, ENNETensorDataType DataType, TArrayView<const int32> Shape, const void* Data, uint64 DataSize) override
	{
		FNNERuntimeRDGDataTensorDesc Desc = GetBaseTensorDesc(Name, Shape, DataType);
		AddInitializer(Desc, Data, DataSize);
		int32 Idx = AddTensor(MoveTemp(Desc));

		return MakeHandle<EHandleType::Tensor>(reinterpret_cast<void*>((int64)Idx));
	}

	virtual FHTensor AddEmptyTensor() override
	{
		int32 Idx = AddTensor(GetEmptyTensorDesc());

		return MakeHandle<EHandleType::Tensor>(reinterpret_cast<void*>((int64)Idx));
	}

	virtual bool AddInput(FHTensor Tensor) override
	{
		int32 Idx = ModelBuilderNNEHelper::NNETensorCast(Tensor);

		if (Idx < 0 || Idx >= Format.Tensors.Num())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add input tensor, invalid tensor index");
			return false;
		}

		if (Format.Tensors[Idx].Type != ENNERuntimeRDGDataTensorType::None)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add input tensor, tensor usage already set up");
			return false;
		}

		Format.Tensors[Idx].Type = ENNERuntimeRDGDataTensorType::Input;

		return true;
	}

	virtual bool AddOutput(FHTensor Tensor) override
	{
		int32 Idx = ModelBuilderNNEHelper::NNETensorCast(Tensor);

		if (Idx < 0 || Idx >= Format.Tensors.Num())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add output tensor, invalid tensor index");
			return false;
		}

		if (Format.Tensors[Idx].Type != ENNERuntimeRDGDataTensorType::None)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add output tensor, tensor usage already set up");
			return false;
		}

		Format.Tensors[Idx].Type = ENNERuntimeRDGDataTensorType::Output;

		return true;
	}

	virtual FHOperator AddOperator(const FString& TypeName, const FString& Domain, TOptional<uint32> Version, const FString& Name = TEXT("")) override
	{
		int32 Idx = Format.Operators.Num();

		FNNERuntimeRDGDataOperatorDesc	Operator{};

		Operator.TypeName = TypeName;
		Operator.DomainName = Domain;
		Operator.Version = Version;
		Format.Operators.Emplace(Operator);

		return MakeHandle<EHandleType::Operator>(reinterpret_cast<void*>((int64)Idx));
	}

	virtual bool AddOperatorInput(FHOperator Op, FHTensor Tensor) override
	{
		int32 OpIdx = ModelBuilderNNEHelper::NNEOperatorCast(Op);
		int32 TensorIdx = ModelBuilderNNEHelper::NNETensorCast(Tensor);

		if (TensorIdx < 0 || TensorIdx >= Format.Tensors.Num())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add operator input tensor, invalid tensor index");
			return false;
		}

		Format.Operators[OpIdx].InTensors.Emplace(TensorIdx);

		return true;
	}

	virtual bool AddOperatorOutput(FHOperator Op, FHTensor Tensor) override
	{
		int32 OpIdx = ModelBuilderNNEHelper::NNEOperatorCast(Op);
		int32 TensorIdx = ModelBuilderNNEHelper::NNETensorCast(Tensor);

		if (TensorIdx < 0 || TensorIdx >= Format.Tensors.Num())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add operator output tensor, invalid tensor index");
			return false;
		}

		if (Format.Tensors[TensorIdx].Type == ENNERuntimeRDGDataTensorType::Input)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Failed to add output tensor, tensor usage already set up to input");
			return false;
		}

		if (Format.Tensors[TensorIdx].Type == ENNERuntimeRDGDataTensorType::None)
		{
			Format.Tensors[TensorIdx].Type = ENNERuntimeRDGDataTensorType::Intermediate;
		}

		Format.Operators[OpIdx].OutTensors.Emplace(TensorIdx);

		return true;
	}

	virtual bool AddOperatorAttribute(FHOperator Op, const FString& Name, const FNNERuntimeRDGDataAttributeValue& Value) override
	{
		int32 OpIdx = ModelBuilderNNEHelper::NNEOperatorCast(Op);

		FNNERuntimeRDGDataAttributeDesc& Attribute = Format.Operators[OpIdx].Attributes.Emplace_GetRef();
		Attribute.Name = Name;
		Attribute.Value = Value;

		return true;
	}

private:

	static FNNERuntimeRDGDataTensorDesc GetBaseTensorDesc(const FString& InName, TArrayView<const int32> InShape, ENNETensorDataType InDataType)
	{
		FNNERuntimeRDGDataTensorDesc&&	Desc{};

		Desc.Name = InName;
		Desc.Shape = InShape;
		Desc.Type = ENNERuntimeRDGDataTensorType::None;
		Desc.DataType = InDataType;

		return Desc;
	}

	void AddInitializer(FNNERuntimeRDGDataTensorDesc& Desc, const void* Data, uint64 DataSize = 0u)
	{
		Desc.Type = ENNERuntimeRDGDataTensorType::Initializer;
		Desc.DataSize = DataSize;

		// Handle empty data initializers, i.e. when DataSize is 0
		if (DataSize)
		{
			Desc.DataOffset = Format.TensorData.AddUninitialized(DataSize);
			FMemory::Memcpy(Format.TensorData.GetData() + Desc.DataOffset, Data, DataSize);
		}
	}

	FString GenerateEmptyTensorName()
	{
		return FString::Printf(TEXT("__NNE_EmptyTensor_%u"), EmptyTensorCounter++);
	}

	FNNERuntimeRDGDataTensorDesc GetEmptyTensorDesc()
	{
		FNNERuntimeRDGDataTensorDesc&&	Desc{};

		Desc.Name = GenerateEmptyTensorName();
		Desc.Shape = { 0 };
		Desc.Type = ENNERuntimeRDGDataTensorType::Empty;
		Desc.DataType = ENNETensorDataType::None;

		return Desc;
	}

	int32 AddTensor(FNNERuntimeRDGDataTensorDesc&& InTensorDesc)
	{
		int32 Idx = -1;

		int32* Val = TensorMap.Find(InTensorDesc.Name);

		if (Val)
		{
			Idx = *Val;
		}
		else
		{
			Format.Tensors.Add(MoveTemp(InTensorDesc));
			Idx = Format.Tensors.Num() - 1;

			TensorMap.Add(Format.Tensors[Idx].Name, Idx);
		}

		return Idx;
	}


	FNNERuntimeRDGDataModelFormat	Format;
	TMap<FString, int32>	TensorMap;
	uint32 EmptyTensorCounter = 0;
};

TUniquePtr<IModelBuilder> CreateNNEModelBuilder()
{
	return MakeUnique<FModelBuilderNNE>();
}

} // namespace UE::NNERuntimeRDGUtils::Private
