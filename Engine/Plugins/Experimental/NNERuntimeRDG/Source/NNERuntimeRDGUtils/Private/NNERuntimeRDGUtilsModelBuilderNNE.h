// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNERuntimeRDGUtilsModelBuilder.h"
#include "Templates/UniquePtr.h"

namespace UE::NNERuntimeRDGUtils::Private
{

TUniquePtr<IModelBuilder> CreateNNEModelBuilder();

} // UE::NNERuntimeRDGUtils::Private

