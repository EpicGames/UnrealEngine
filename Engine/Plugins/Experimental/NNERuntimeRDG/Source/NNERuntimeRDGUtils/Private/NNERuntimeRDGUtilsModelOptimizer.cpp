// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsModelOptimizer.h"

#include "NNERuntimeRDGUtilsModelOptimizerNNE.h"

namespace UE::NNERuntimeRDGUtils::Internal
{

TUniquePtr<IModelOptimizer> CreateModelOptimizer()
{
	return MakeUnique<Private::FModelOptimizerONNXToNNERT>();
}

} // namespace UE::NNERuntimeRDGUtils::Internal
