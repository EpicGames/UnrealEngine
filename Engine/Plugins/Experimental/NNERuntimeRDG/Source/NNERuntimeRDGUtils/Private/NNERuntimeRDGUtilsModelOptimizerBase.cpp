// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsModelOptimizerBase.h"

#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataFormat.h"

THIRD_PARTY_INCLUDES_START
#include "onnx/common/common.h"
#include "onnx/checker.h"
#include "onnx/proto_utils.h"
THIRD_PARTY_INCLUDES_END

namespace UE::NNERuntimeRDGUtils::Private
{

FString FModelValidatorONNX::GetName() const
{
	return TEXT("ONNX Model validator");
}

bool FModelValidatorONNX::ValidateModel(TConstArrayView<uint8> InputModel) const
{
	onnx::ModelProto Model;
	onnx::ParseProtoFromBytes(&Model, reinterpret_cast<const char*>(InputModel.GetData()), static_cast<size_t>(InputModel.Num()));

#ifdef ONNX_NO_EXCEPTIONS
	static_assert(false, "ONNX_NO_EXCEPTIONS is defined meaning onnx check_model would abort the program in case of validation failure.");
#else
	try
	{
		onnx::checker::check_model(Model);
	}
	catch (onnx::checker::ValidationError& e)
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Input model is invalid : %ls.", ANSI_TO_TCHAR(e.what()));
		return false;
	}
#endif

	return true;
}

bool FModelOptimizerBase::IsModelValid(TConstArrayView<uint8> ModelToValidate)
{
	bool bIsModelValid = true;

	for (TSharedPtr<Internal::IModelValidator>& Validator : Validators)
	{
		check(Validator.IsValid());
		if (!Validator->ValidateModel(ModelToValidate))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Model validator '%ls' detected an error.", *(Validator->GetName()));
			bIsModelValid = false;
		}
	}
	return bIsModelValid;
}

bool FModelOptimizerBase::ApplyAllPassesAndValidations(TArray<uint8>& OptimizedModel)
{
	if (!IsModelValid(OptimizedModel))
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Model is not valid.");
		return false;
	}
		
	for (TSharedPtr<Internal::IModelOptimizerPass>& Pass : OptimizationPasses)
	{
		check(Pass.IsValid());

		//Note: Useful to enable for debug purpose
		//FFileHelper::SaveArrayToFile(OptimizedModel.Data, TEXT("D:\\OnnxBeforePass.onnx"));
			
		if (!Pass->ApplyPass(OptimizedModel))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Error while executing model optimisation pass '%ls'.", *(Pass->GetName()));
			return false;
		}

		//Note: Useful to enable for debug purpose
		//FFileHelper::SaveArrayToFile(OptimizedModel.Data, TEXT("D:\\OnnxAfterPass.onnx"));

		if (!IsModelValid(OptimizedModel))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Model validation failed after optimisation pass '%ls'.", *(Pass->GetName()));
			return false;
		}
	}

	return true;
}

void FModelOptimizerBase::AddOptimizationPass(TSharedPtr<Internal::IModelOptimizerPass> ModelOptimizerPass)
{
	if (ModelOptimizerPass.IsValid())
	{
		OptimizationPasses.Add(ModelOptimizerPass);
	}
}

void FModelOptimizerBase::AddValidator(TSharedPtr<Internal::IModelValidator> ModelValidator)
{
	if (ModelValidator.IsValid())
	{
		Validators.Add(ModelValidator);
	}
}

bool FModelOptimizerBase::Optimize(TConstArrayView<uint8> InputModel, TArray<uint8>& OutModel)
{
	OutModel = InputModel;
	return ApplyAllPassesAndValidations(OutModel);
}

} // namespace UE::NNERuntimeRDGUtils::Private
