// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGUtilsModelOptimizerInterface.h"
#include "Templates/SharedPointer.h"

namespace UE::NNERuntimeRDGUtils::Private
{
class FModelValidatorONNX : public Internal::IModelValidator
{
public:
	virtual FString GetName() const override;
	virtual bool ValidateModel(TConstArrayView<uint8> InputModel) const override;
};

class FModelOptimizerBase : public Internal::IModelOptimizer
{
protected:
	FModelOptimizerBase() {}
	TArray<TSharedPtr<Internal::IModelOptimizerPass>> OptimizationPasses;
	TArray<TSharedPtr<Internal::IModelValidator>> Validators;

	bool IsModelValid(TConstArrayView<uint8> ModelToValidate);
	bool ApplyAllPassesAndValidations(TArray<uint8>& OptimizedModel);

public:
	virtual void AddOptimizationPass(TSharedPtr<Internal::IModelOptimizerPass> ModelOptimizerPass) override;
	virtual void AddValidator(TSharedPtr<Internal::IModelValidator> ModelValidator) override;
	virtual bool Optimize(TConstArrayView<uint8> InputModel, TArray<uint8>& OutModel) override;
};

} // namespace UE::NNERuntimeRDGUtils::Private
