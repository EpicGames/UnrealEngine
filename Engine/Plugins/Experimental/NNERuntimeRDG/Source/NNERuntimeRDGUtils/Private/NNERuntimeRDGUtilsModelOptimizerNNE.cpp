// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsModelOptimizerNNE.h"

#include "Containers/ContainersFwd.h"
#include "NNEHlslShadersLog.h"
#include "NNERuntimeRDGDataAttributeTensor.h"
#include "NNERuntimeRDGDataFormat.h"
#include "NNERuntimeRDGUtilsModelBuilderNNE.h"
#include "NNERuntimeRDGUtilsModelOptimizerONNX.h"
#include "NNERuntimeRDGUtilsHelpers.h"
#include "NNETypes.h"

THIRD_PARTY_INCLUDES_START
#include <onnx/onnx_pb.h>
#include <onnx/shape_inference/implementation.h>
#include <onnx/defs/schema.h>
THIRD_PARTY_INCLUDES_END

namespace UE::NNERuntimeRDGUtils::Private
{

namespace ModelOptimizerNNEHelper
{
	ENNETensorDataType GetNNETensorTypeFromONNX(int32 DataType)
	{
		switch (DataType)
		{
		case onnx::TensorProto_DataType_UNDEFINED:	return ENNETensorDataType::None;
		case onnx::TensorProto_DataType_FLOAT:		return ENNETensorDataType::Float;
		case onnx::TensorProto_DataType_UINT8:		return ENNETensorDataType::UInt8;
		case onnx::TensorProto_DataType_INT8:		return ENNETensorDataType::Int8;
		case onnx::TensorProto_DataType_UINT16:		return ENNETensorDataType::UInt16;
		case onnx::TensorProto_DataType_INT16:		return ENNETensorDataType::Int16;
		case onnx::TensorProto_DataType_INT32:		return ENNETensorDataType::Int32;
		case onnx::TensorProto_DataType_INT64:		return ENNETensorDataType::Int64;
		//case onnx::TensorProto_DataType_STRING:	return ENNETensorDataType::String;
		case onnx::TensorProto_DataType_BOOL:		return ENNETensorDataType::Boolean;
		case onnx::TensorProto_DataType_FLOAT16:	return ENNETensorDataType::Half;
		case onnx::TensorProto_DataType_DOUBLE:		return ENNETensorDataType::Double;
		case onnx::TensorProto_DataType_UINT32:		return ENNETensorDataType::UInt32;
		case onnx::TensorProto_DataType_UINT64:		return ENNETensorDataType::UInt64;
		case onnx::TensorProto_DataType_COMPLEX64:	return ENNETensorDataType::Complex64;
		case onnx::TensorProto_DataType_COMPLEX128:	return ENNETensorDataType::Complex128;
		case onnx::TensorProto_DataType_BFLOAT16:	return ENNETensorDataType::BFloat16;
		default: return ENNETensorDataType::None;
		}
	}

	int32 ConvertInt32AttributeValueFromONNX(int32 ONNXValue, const FString& AttributeName, const FString& OpName)
	{
		if (OpName == TEXT("Cast") && AttributeName == TEXT("to"))
		{
			//Cast.to attribute follow TensorProto.DataType
			//however NNE format is agnostic to TensorProto thus we need a conversion.
			return (int32)GetNNETensorTypeFromONNX(ONNXValue);
		}
		return ONNXValue;
	}

	const onnx::TensorProto* GetInitializerFromGraphProto(const onnx::GraphProto& Graph, const std::string& Name)
	{
		for (const onnx::TensorProto& Initializer : Graph.initializer())
		{
			const std::string& InitializerName = Initializer.name();

			if (InitializerName == Name)
			{
				return &Initializer;
			}
		}
		return nullptr;
	}

	const onnx::ValueInfoProto* GetValueInfoProtoFromGraphProto(const onnx::GraphProto& Graph, const std::string& Name)
	{
		for (const onnx::ValueInfoProto& Tensor : Graph.input())
		{
			const std::string& TensorName = Tensor.name();

			if (TensorName == Name)
			{
				return &Tensor;
			}
		}

		for (const onnx::ValueInfoProto& Tensor : Graph.output())
		{
			const std::string& TensorName = Tensor.name();

			if (TensorName == Name)
			{
				return &Tensor;
			}
		}

		for (const onnx::ValueInfoProto& Tensor : Graph.value_info())
		{
			const std::string& TensorName = Tensor.name();

			if (TensorName == Name)
			{
				return &Tensor;
			}
		}

		return nullptr;
	}

	class TensorInfoData
	{
		const uint8* Data = nullptr;
		size_t DataSize = 0;
		uint32 ElementSize = 0;
		uint32 ElementStride = 0;
		TArray<uint8_t> PackedData;

	public:

		TensorInfoData() {}

		static TensorInfoData Make(const void* Data, size_t DataSize, uint32 ElementSize, uint32 ElementStride)
		{
			check(Data);
			check(ElementStride > 0);
			check(DataSize % ElementStride == 0);
			check(ElementSize <= ElementStride);

			TensorInfoData Result = TensorInfoData();
			Result.Data = reinterpret_cast<const uint8 *>(Data);
			Result.DataSize = DataSize;
			Result.ElementSize = ElementSize;
			Result.ElementStride = ElementStride;

			return Result;
		}

		TConstArrayView<uint8> GetArrayView()
		{
			if (ElementStride > ElementSize)
			{
				if (PackedData.IsEmpty())
				{
					PackedData.Reset(DataSize / ElementStride * ElementSize);
					for (uint64 DataIndex = 0; DataIndex < DataSize; DataIndex += ElementStride)
					{
						for (uint64 Offset = 0; Offset < ElementSize; ++Offset)
						{
							PackedData.Add(Data[DataIndex + Offset]);
						}
					}
				}
				return MakeArrayView(PackedData);
			}
			else
			{
				return MakeArrayView(Data, DataSize);
			}
		}
	};

	bool GetTensorInfoFromONNXInitializer(const onnx::TensorProto& Tensor, TArray<int32>& Shape, ENNETensorDataType& DataType, TensorInfoData& OutData)
	{
		DataType = GetNNETensorTypeFromONNX(Tensor.data_type());

		Shape.Reset(0);
		for (int32 i = 0; i < Tensor.dims_size(); ++i)
		{
			Shape.Add(static_cast<int32>(Tensor.dims(i)));
		}

		const uint32 ElementSize = UE::NNE::GetTensorDataTypeSizeInBytes(DataType);
		const void *Data = nullptr;
		size_t DataSize = 0;
		uint32 ElementStride;

		if (Tensor.has_raw_data())
		{
			const std::string& RawData = Tensor.raw_data();
			Data = RawData.c_str();
			DataSize = RawData.size();
			ElementStride = ElementSize;
		}
		else if (Tensor.double_data().size())
		{
			//DOUBLE or COMPLEX128
			Data = Tensor.double_data().data();
			DataSize = Tensor.double_data().size() * sizeof(double);
			ElementStride = sizeof(double);
		}
		else if (Tensor.external_data().size()) 
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "External data not supported.");
			return false;
		}
		else if (Tensor.float_data().size())
		{
			//FLOAT or COMPLEX64
			Data = Tensor.float_data().data();
			DataSize = Tensor.float_data().size() * sizeof(float);
			ElementStride = sizeof(float);
		}
		else if (Tensor.int32_data().size())
		{
			//Supported : INT32, FLOAT16
			//Not supported at the moment: INT16, INT8, UINT16, UINT8, BOOL, BFLOAT16, FLOAT8E4M3FN, FLOAT8E4M3FNUZ, FLOAT8E5M2, FLOAT8E5M2FNUZ, UINT32
			if (DataType != ENNETensorDataType::Int32 &&
				DataType != ENNETensorDataType::Half)
			{
				return false;
			}
			Data = Tensor.int32_data().data();
			DataSize = Tensor.int32_data().size() * sizeof(int32);
			ElementStride = sizeof(int32);
		}
		else if (Tensor.uint64_data().size())
		{
			//Supported UINT64
			Data = Tensor.uint64_data().data();
			DataSize = Tensor.uint64_data().size() * sizeof(uint64);
			ElementStride = sizeof(uint64);
		}
		else if (Tensor.int64_data().size())
		{
			//Supported INT64
			Data = Tensor.int64_data().data();
			DataSize = Tensor.int64_data().size() * sizeof(int64);
			ElementStride = sizeof(int64);
		}
		else
		{
			return false;
		}
		OutData = TensorInfoData::Make(Data, DataSize, ElementSize, ElementStride);
		return true;
	}

	void GetTensorInfoFromONNXValueInfo(const onnx::ValueInfoProto& Tensor, TArray<int32>& Shape, ENNETensorDataType& DataType)
	{
		const onnx::TypeProto& Type = Tensor.type();
		const onnx::TypeProto_Tensor& TensorType = Type.tensor_type();
		const onnx::TensorShapeProto& TensorShape = TensorType.shape();
		DataType = GetNNETensorTypeFromONNX(TensorType.elem_type());

		Shape.Reset(0);
		for (const onnx::TensorShapeProto_Dimension& Dim : TensorShape.dim())
		{
			Shape.Add(static_cast<int32>(Dim.dim_value()));
		}
	}

	static bool GetAttributeTensorFromONNXInitializer(const onnx::TensorProto& InTensor, NNERuntimeRDGData::Internal::FAttributeTensor& OutTensor)
	{
		using NNE::FTensorShape;
		using NNERuntimeRDGData::Internal::FAttributeTensor;

		ENNETensorDataType DataType = ENNETensorDataType::None;
		TArray<int32> Shape;
		TensorInfoData Data;

		if (!GetTensorInfoFromONNXInitializer(InTensor, Shape, DataType, Data))
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor data could not be loaded");
			return false;
		}
		if (Shape.Num() > FTensorShape::MaxRank)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor shape of rank %i exceeds MaxRank %i", Shape.Num(), FTensorShape::MaxRank);
			return false;
		}
		TArray<uint32, TInlineAllocator<FTensorShape::MaxRank>> UIntShape;
		for (int32 value : Shape)
		{
			if (value < 0)
			{
				UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor shape has negative value");
				return false;
			}
			UIntShape.Add(value);
		}
		const FTensorShape TensorShape = FTensorShape::Make(UIntShape);
		TConstArrayView<uint8> DataView = Data.GetArrayView();
		uint32 ExpectedDataSize = TensorShape.Volume() * UE::NNE::GetTensorDataTypeSizeInBytes(DataType);
		if (ExpectedDataSize != DataView.NumBytes())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Tensor data size %zu doesn't match expected data size %i",
				   					DataView.NumBytes(), ExpectedDataSize);
			return false;
		}
		OutTensor = FAttributeTensor::Make(TensorShape, DataType, DataView);
		return true;
	}

	bool BuildNNEFormatFromONNX(TArray<uint8>& ONNXData, TArray<uint8>& NNEData)
	{
		TUniquePtr<IModelBuilder> Builder = CreateNNEModelBuilder();

		onnx::ModelProto ModelProto;
		const bool result = ModelProto.ParseFromArray(ONNXData.GetData(), ONNXData.Num());
		if (!result)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Could not parse the input model as a ModelProto.");
			return false;
		}

		if (ModelProto.opset_import_size() < 1)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Could not read opset version from ONNX.");
			return false;
		}

		//Run shape inference as we need shapes informations to convert to NNE runtime format
		onnx::shape_inference::InferShapes(ModelProto);

		const onnx::GraphProto& Graph = ModelProto.graph();

		Builder->Begin(ANSI_TO_TCHAR(Graph.name().c_str()));

		// Add tensors for graph inputs
		for (const onnx::ValueInfoProto& Input : Graph.input())
		{
			// ONNX GraphProto sometime return initializers as input,
			// we skip them here as we only want user providable inputs as NNE inputs
			if (GetInitializerFromGraphProto(Graph, Input.name())  != nullptr)
			{
				continue;
			}

			ENNETensorDataType DataType;
			TArray<int32> Shape;
			GetTensorInfoFromONNXValueInfo(Input, Shape, DataType);
			IModelBuilder::FHTensor Tensor = Builder->AddTensor(ANSI_TO_TCHAR(Input.name().c_str()), DataType, Shape);

			Builder->AddInput(Tensor);
		}

		// Add tensors for graph outputs
		for (const onnx::ValueInfoProto& Output : Graph.output())
		{
			ENNETensorDataType DataType;
			TArray<int32> Shape;
			GetTensorInfoFromONNXValueInfo(Output, Shape, DataType);
			IModelBuilder::FHTensor Tensor = Builder->AddTensor(ANSI_TO_TCHAR(Output.name().c_str()), DataType, Shape);

			Builder->AddOutput(Tensor);

			const onnx::TensorProto* Initializer = GetInitializerFromGraphProto(Graph, Output.name());
			if (Initializer)
			{
				ENNETensorDataType InitializerDataType;
				TArray<int32> InitializerShape;
				TensorInfoData Data;

				if (!GetTensorInfoFromONNXInitializer(*Initializer, InitializerShape, InitializerDataType, Data))
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor data could not be loaded for weights of output node '%ls'", ANSI_TO_TCHAR(Output.name().c_str()));
					return false;
				}

				if (DataType != InitializerDataType)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Initializer type does not match output type for output tensor %ls.", ANSI_TO_TCHAR(Output.name().c_str()));
					return false;
				}


				if (Shape != InitializerShape)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Initializer shape does not match output shape for output tensor %ls.", ANSI_TO_TCHAR(Output.name().c_str()));
					return false;
				}

				TConstArrayView<uint8> DataView = Data.GetArrayView();
				IModelBuilder::FHTensor TensorInitializer =
					Builder->AddConstantTensor(FString(ANSI_TO_TCHAR(Output.name().c_str())) + TEXT("_NNEInitializer"), DataType, Shape, DataView.GetData(), DataView.NumBytes());

				const FString IdentityOpType = TEXT("Identity");
				TOptional<uint32> OpVersion = GetOpVersionFromOpsetVersion(IdentityOpType, (int) ModelProto.opset_import(0).version());
				if(!OpVersion.IsSet())
				{
					return false;
				}
				IModelBuilder::FHOperator Op = Builder->AddOperator(IdentityOpType, OnnxDomainName, *OpVersion);
				Builder->AddOperatorInput(Op, TensorInitializer);
				Builder->AddOperatorOutput(Op, Tensor);
			}

		}

		// Traverse all the nodes get their inputs, outputs and tensor data
		for (const onnx::NodeProto& Node : Graph.node())
		{
			const std::string& OnnxOpType = Node.op_type();
			FString NNEOpType(StringCast<TCHAR>(OnnxOpType.c_str()));
			const std::string& OnnxOpName = Node.name();
			FString NNEOpName(StringCast<TCHAR>(OnnxOpName.c_str()));
			TOptional<uint32> OpVersion = GetOpVersionFromOpsetVersion(NNEOpType, (int) ModelProto.opset_import(0).version());
			if(!OpVersion.IsSet())
			{
				return false;
			}
			IModelBuilder::FHOperator Op = Builder->AddOperator(NNEOpType, OnnxDomainName, *OpVersion, NNEOpName);

			for (const onnx::AttributeProto& Attribute : Node.attribute())
			{
				FString AttributeName = ANSI_TO_TCHAR(Attribute.name().c_str());

				if (Attribute.type() == onnx::AttributeProto::FLOAT)
				{
					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Attribute.f()));
				}
				else if (Attribute.type() == onnx::AttributeProto::INT)
				{
					int32 NNEAttributeValue = ConvertInt32AttributeValueFromONNX((int32)Attribute.i(), AttributeName, NNEOpType);
					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(NNEAttributeValue));
				}
				else if (Attribute.type() == onnx::AttributeProto::INTS)
				{
					TArray<int32> Values;
					Values.Reserve(Attribute.ints_size());
					for (int64 Value64 : Attribute.ints())
					{
						int64 ValueClamped = FMath::Clamp<int64>(Value64, MIN_int32, MAX_int32);
						if (ValueClamped != Value64)
						{
							UE_LOGF(LogNNERuntimeRDGHlsl, Display, "Overflow detected when converting to int32 attribute '%ls' in node '%ls'", *AttributeName, *NNEOpType);
						}
						
						Values.Add((int32)ValueClamped);
					}

					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Values));
				}
				else if (Attribute.type() == onnx::AttributeProto::STRING)
				{
					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(FString(ANSI_TO_TCHAR(Attribute.s().c_str()))));
				}
				else if (Attribute.type() == onnx::AttributeProto::STRINGS)
				{
					TArray<FString> Values;
					Values.Reserve(Attribute.strings_size());
					for (const std::string& Value : Attribute.strings())
					{
						Values.Add(FString(UTF8_TO_TCHAR(Value.c_str())));
					}

					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Values));
				}
				else if (Attribute.type() == onnx::AttributeProto::FLOATS)
				{
					TArray<float> Values;
					Values.Reserve(Attribute.floats_size());
					for (float Value : Attribute.floats())
					{
						Values.Add(Value);
					}

					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Values));
				}
				else if (Attribute.type() == onnx::AttributeProto::TENSOR)
				{
					using NNERuntimeRDGData::Internal::FAttributeTensor;

					FAttributeTensor Tensor;
					if (!GetAttributeTensorFromONNXInitializer(Attribute.t(), Tensor))
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Failed to get data from attribute %ls in operator %ls", *AttributeName, *NNEOpName);
						return false;
					}
					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Tensor));
				}
				else if (Attribute.type() == onnx::AttributeProto::TENSORS)
				{
					using NNERuntimeRDGData::Internal::FAttributeTensor;

					TArray<FAttributeTensor> Values;

					int TensorIndex = 0;
					for (const onnx::TensorProto& Initializer : Attribute.tensors())
					{
						FAttributeTensor Tensor;
						if (!GetAttributeTensorFromONNXInitializer(Initializer, Tensor))
						{
							UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Failed to get data from tensor at index %i of attribute %ls in operator %ls", TensorIndex, *AttributeName, *NNEOpName);
							return false;
						}
						Values.Add(Tensor);
						TensorIndex++;
					}
					Builder->AddOperatorAttribute(Op, AttributeName, FNNERuntimeRDGDataAttributeValue(Values));
				}
				else
				{
					//Note: Would be good to have better error reporting by adding type (example: sparse tensor)
					UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Unsupported attribute type for attribute '%ls' in node '%ls' of type '%ls'", *AttributeName, *NNEOpName, *NNEOpType);
				}
			}

			for (const std::string& TensorName : Node.input())
			{
				ENNETensorDataType DataType = ENNETensorDataType::None;
				TArray<int32> Shape;
				TensorInfoData Data;

				IModelBuilder::FHTensor Tensor;

				const onnx::TensorProto* Initializer = GetInitializerFromGraphProto(Graph, TensorName);
				if (Initializer)
				{
					if (!GetTensorInfoFromONNXInitializer(*Initializer, Shape, DataType, Data))
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Tensor data could not be loaded for weight '%ls' in node '%ls' of type '%ls'", ANSI_TO_TCHAR(TensorName.c_str()), *NNEOpName, *NNEOpType);
						return false;
					}
					TConstArrayView<uint8> DataView = Data.GetArrayView();
					Tensor = Builder->AddConstantTensor(ANSI_TO_TCHAR(TensorName.c_str()), DataType, Shape, DataView.GetData(), DataView.NumBytes());
				}
				else if (!TensorName.empty())
				{
					const onnx::ValueInfoProto* ValueInfoProto = GetValueInfoProtoFromGraphProto(Graph, TensorName);
					if (!ValueInfoProto)
					{
						UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Could not find Tensor ValueInfoProto or Initializer in graph for input '%ls' in node '%ls' of type '%ls'", ANSI_TO_TCHAR(TensorName.c_str()), *NNEOpName, *NNEOpType);
						return false;
					}

					GetTensorInfoFromONNXValueInfo(*ValueInfoProto, Shape, DataType);
					Tensor = Builder->AddTensor(ANSI_TO_TCHAR(TensorName.c_str()), DataType, Shape);
				}
				else // Empty tensor
				{
					Tensor = Builder->AddEmptyTensor();
				}

				Builder->AddOperatorInput(Op, Tensor);
			}

			for (const std::string& TensorName : Node.output())
			{
				const onnx::ValueInfoProto* ValueInfoProto = GetValueInfoProtoFromGraphProto(Graph, TensorName);
				if (!ValueInfoProto)
				{
					UE_LOGF(LogNNERuntimeRDGHlsl, Error, "Could not find Tensor ValueInfoProto in graph for output '%ls' in node '%ls' of type '%ls'", ANSI_TO_TCHAR(TensorName.c_str()), *NNEOpName, *NNEOpType);
					return false;
				}
				ENNETensorDataType DataType;
				TArray<int32> Shape;
				GetTensorInfoFromONNXValueInfo(*ValueInfoProto, Shape, DataType);

				IModelBuilder::FHTensor Tensor = Builder->AddTensor(ANSI_TO_TCHAR(TensorName.c_str()), DataType, Shape);

				Builder->AddOperatorOutput(Op, Tensor);
			}
		}

		return Builder->End(NNEData);
	}

} // namespace ModelOptimizerNNEHelper

bool FModelOptimizerONNXToNNERT::Optimize(TConstArrayView<uint8> InputModel, TArray<uint8>& OptimizedModel)
{
	OptimizedModel.Reset();

	TArray<uint8> OptimizedONNXModel;
	FModelOptimizerONNXToONNX ModelOptimizerONNXToONNX;

	if (!ModelOptimizerONNXToONNX.Optimize(InputModel, OptimizedONNXModel))
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Error while optimizing the ONNX model before convertion to NNERT format.");
		return false;
	}

	if (!ModelOptimizerNNEHelper::BuildNNEFormatFromONNX(OptimizedONNXModel, OptimizedModel))
	{
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Error while building NNERT Model from ONNX.");
		return false;
	}

	return ApplyAllPassesAndValidations(OptimizedModel);
}

} // namespace UE::NNERuntimeRDGUtils::Private
