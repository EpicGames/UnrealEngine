// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGUtilsModelOptimizerBase.h"

namespace UE::NNERuntimeRDGUtils::Private
{

class FModelOptimizerONNXToNNERT : public FModelOptimizerBase
{
public:
	virtual FString GetName() const override 
	{
		return TEXT("NNEModelOptimizerFromONNXToNNERuntimeRDGHlsl");
	}

	virtual bool Optimize(TConstArrayView<uint8> InputModel, TArray<uint8>& OutModel) override;
};

} // namespace UE::NNERuntimeRDGUtils::Private
