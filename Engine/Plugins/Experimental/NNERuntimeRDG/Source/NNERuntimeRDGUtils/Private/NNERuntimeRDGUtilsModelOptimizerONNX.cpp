// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeRDGUtilsModelOptimizerONNX.h"

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNEHlslShadersLog.h"
#include "NNEOnnxruntimeEditor.h"

THIRD_PARTY_INCLUDES_START
#include <onnx/onnx_pb.h>
#include <onnx/shape_inference/implementation.h>
THIRD_PARTY_INCLUDES_END

namespace UE::NNERuntimeRDGUtils::Private
{

class FOnnxRuntimeModelOptimizerPass : public Internal::IModelOptimizerPass
{
public:
	FOnnxRuntimeModelOptimizerPass()
	{
	}

	virtual FString GetName() const
	{
		return TEXT("Onnx runtime model optimization");
	}

	virtual bool ApplyPass(TArray<uint8>& ModelData) const
	{
		onnx::ModelProto ModelProto;
		const bool result = ModelProto.ParseFromArray(ModelData.GetData(), ModelData.Num());
		if (!result)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls could not parse the input model as a ModelProto.", *(GetName()));
			return false;
		}

		static const auto CVarHlslModelOptimization = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("nne.hlsl.ModelOptimization"));
		if (CVarHlslModelOptimization && CVarHlslModelOptimization->GetValueOnAnyThread() == 0)
		{
			return true;
		}
		
		// We disable ONNX Runtime optimizations if a model uses FP16 Tensors 
		// since these optimizations would add cast operators from and to FP16
		// at the beginning and end of the network and convert all other operators to FP32.
		if (HasFP16Tensor(ModelProto))
		{
			return true;
		}

		FString ProjIntermediateDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
		FString ModelToOptimizePath = FPaths::CreateTempFilename(*ProjIntermediateDir, TEXT("ORTOptimizerPass_ToOptimize"), TEXT(".onnx"));
		FString ModelOptimizedPath = FPaths::CreateTempFilename(*ProjIntermediateDir, TEXT("ORTOptimizerPass_Optimized"), TEXT(".onnx"));

		//See https://onnxruntime.ai/docs/performance/model-optimizations/graph-optimizations.html
		//We only enable all the optimization when going to ORT format itself for the CPU provider
		GraphOptimizationLevel OptimizationLevel = ORT_ENABLE_BASIC;
		FFileHelper::SaveArrayToFile(ModelData, *ModelToOptimizePath);
		{
			Ort::ThreadingOptions ThreadingOptions;
			ThreadingOptions.SetGlobalIntraOpNumThreads(1);
			ThreadingOptions.SetGlobalInterOpNumThreads(1);

			Ort::Env Env(ThreadingOptions);

			Ort::SessionOptions SessionOptions;
			SessionOptions.DisablePerSessionThreads();
			SessionOptions.SetGraphOptimizationLevel(OptimizationLevel);
			#if PLATFORM_WINDOWS
				SessionOptions.SetOptimizedModelFilePath(*ModelOptimizedPath);

				Ort::Session Session(Env, *ModelToOptimizePath, SessionOptions);
			#else
				SessionOptions.SetOptimizedModelFilePath(TCHAR_TO_ANSI(*ModelOptimizedPath));

				Ort::Session Session(Env, TCHAR_TO_ANSI(*ModelToOptimizePath), SessionOptions);
			#endif
		}
		FFileHelper::LoadFileToArray(ModelData, *ModelOptimizedPath);

		IFileManager::Get().Delete(*ModelToOptimizePath);
		IFileManager::Get().Delete(*ModelOptimizedPath);

		return true;
	}
private:
	bool HasFP16Tensor(const onnx::ModelProto& Model) const
	{
		const onnx::GraphProto& Graph = Model.graph();
		for (const onnx::TensorProto& Tensor : Graph.initializer())
		{
			if (Tensor.data_type() == ONNXTensorElementDataType::ONNX_TENSOR_ELEMENT_DATA_TYPE_FLOAT16)
			{
				return true;
			}
		}

		for (onnx::ValueInfoList Tensors : {Graph.input(), Graph.output()})
		{
			for (const onnx::ValueInfoProto& Tensor : Tensors)
			{
				if (!Tensor.has_type())
				{
					continue;
				}
				const onnx::TypeProto Type = Tensor.type();
				if (!Type.has_tensor_type())
				{
					continue;
				}
				const onnx::TypeProto_Tensor TensorType = Type.tensor_type();
				if (!TensorType.has_elem_type())
				{
					continue;
				}
				if (TensorType.elem_type() == ONNXTensorElementDataType::ONNX_TENSOR_ELEMENT_DATA_TYPE_FLOAT16)
				{
					return true;
				}
			}
		}

		return false;
	}

};

class FOnnxDomainCleanupModelOptimizerPass : public Internal::IModelOptimizerPass
{
public:
	virtual FString GetName() const
	{
		return TEXT("Onnx domain cleanup");
	}

	virtual bool ApplyPass(TArray<uint8>& ModelData) const
	{
		onnx::ModelProto ModelProto;
		const bool result = ModelProto.ParseFromArray(ModelData.GetData(), ModelData.Num());
		if (!result)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls could not parse the input model as a ModelProto.", *(GetName()));
			return false;
		}

		TArray<FString> UsedDomains;
		const onnx::GraphProto& graph = ModelProto.graph();
		for (const onnx::NodeProto& node : graph.node())
		{
			const char* DomainPtr = node.domain().c_str();
			FString Domain = DomainPtr;
			UsedDomains.AddUnique(Domain);
		}

		google::protobuf::RepeatedPtrField<onnx::OperatorSetIdProto> UsedOperatorSet;
		for (const onnx::OperatorSetIdProto& OpSet : ModelProto.opset_import())
		{
			const char* DomainPtr = OpSet.domain().c_str();
			FString Domain = DomainPtr;
			bool IsUsed = UsedDomains.Contains(Domain);
			// We additionally add the OpSet from models that don't have any UsedDomains/Nodes
			// since we otherwise would get an invalid model
			if (IsUsed || UsedDomains.IsEmpty())
			{
				UsedOperatorSet.Add()->CopyFrom(OpSet);
			}
		}

		ModelProto.mutable_opset_import()->Clear();
		ModelProto.mutable_opset_import()->Add(UsedOperatorSet.cbegin(), UsedOperatorSet.cend());

		ModelData.SetNumUninitialized(ModelProto.ByteSizeLong());
		ModelProto.SerializeToArray(ModelData.GetData(), ModelData.Num());

		return true;
	}

};

class FOnnxShapeInferenceModelOptimizerPass : public Internal::IModelOptimizerPass
{
public:
	virtual FString GetName() const
	{
		return TEXT("Onnx shape inference");
	}

	virtual bool ApplyPass(TArray<uint8>& ModelData) const
	{
		onnx::ModelProto ModelProto;
		const bool result = ModelProto.ParseFromArray(ModelData.GetData(), ModelData.Num());
		if (!result)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "%ls could not parse the input model as a ModelProto.", *(GetName()));
			return false;
		}

#ifdef ONNX_NO_EXCEPTIONS
		UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "ONNX Shape inference can't be run as exception are disabled.");
		return true;
#else

		static onnx::OpSchemaRegistry* OnnxSchemaRegistry = onnx::OpSchemaRegistry::Instance();

		try
		{
			onnx::shape_inference::InferShapes(ModelProto, OnnxSchemaRegistry);
		}
		catch (onnx::InferenceError& e)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Warning, "Shape inference failed with : %ls.", ANSI_TO_TCHAR(e.what()));
		}
#endif
		
		ModelProto.SerializeToArray(ModelData.GetData(), ModelData.Num());

		return true;
	}

};

FModelOptimizerONNXToONNX::FModelOptimizerONNXToONNX()
{
	AddOptimizationPass(MakeShared<FOnnxRuntimeModelOptimizerPass>());
	AddOptimizationPass(MakeShared<FOnnxDomainCleanupModelOptimizerPass>());
	AddOptimizationPass(MakeShared<FOnnxShapeInferenceModelOptimizerPass>());
	AddValidator(MakeShared<FModelValidatorONNX>());
}

} // namespace UE::NNERuntimeRDGUtils::Private
