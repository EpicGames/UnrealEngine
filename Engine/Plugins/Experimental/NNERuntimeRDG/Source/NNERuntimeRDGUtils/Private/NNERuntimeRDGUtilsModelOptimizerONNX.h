// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "NNERuntimeRDGUtilsModelOptimizerBase.h"

namespace UE::NNERuntimeRDGUtils::Private
{

class FModelOptimizerONNXToONNX : public FModelOptimizerBase
{
public:
	FModelOptimizerONNXToONNX();

	virtual FString GetName() const override
	{
		return TEXT("NNEModelOptimizerFromONNXToONNX");
	}
};

} // namespace UE::NNERuntimeRDGUtils::Private
