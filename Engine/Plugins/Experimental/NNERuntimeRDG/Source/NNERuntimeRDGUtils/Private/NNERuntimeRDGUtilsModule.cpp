// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NNEHlslShadersLog.h"
#include "NNEOnnxruntimeEditor.h"

class FNNERuntimeRDGUtilsModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		const FString PluginDir = IPluginManager::Get().FindPlugin("NNERuntimeRDG")->GetBaseDir();
		const FString OrtSharedLibPath = FPaths::Combine(PluginDir, TEXT(UE_STRINGIZE(ONNXRUNTIME_SHAREDLIB_PATH)));

		OrtDllHandle = FPlatformProcess::GetDllHandle(*OrtSharedLibPath);
		if (!OrtDllHandle)
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Fatal, "Failed to load ONNX Runtime shared library!");
			return;
		}

		TUniquePtr<UE::NNEOnnxruntime::OrtApiFunctions> OrtApiFunctions = UE::NNEOnnxruntime::LoadApiFunctions(OrtDllHandle);
		if (!OrtApiFunctions.IsValid())
		{
			UE_LOGF(LogNNERuntimeRDGHlsl, Fatal, "Failed to load ONNX Runtime shared library functions!");
			return;
		}

		Ort::InitApi(OrtApiFunctions->OrtGetApiBase()->GetApi(ORT_API_VERSION));
	}

	virtual void ShutdownModule() override
	{
		if (OrtDllHandle)
		{
			FPlatformProcess::FreeDllHandle(OrtDllHandle);
		}
	}

	void* OrtDllHandle = nullptr;
};

IMPLEMENT_MODULE(FNNERuntimeRDGUtilsModule, NNERuntimeRDGUtils)